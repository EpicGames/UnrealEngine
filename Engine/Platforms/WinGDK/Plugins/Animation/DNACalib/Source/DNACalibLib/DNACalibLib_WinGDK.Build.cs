// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DNACalibLib_WinGDK : DNACalibLib
	{
		public DNACalibLib_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDefinitions.Add("TERSE_ENABLE_SSE=1");
		}
	}
}
