{
	"FileVersion": 3,
	"Version": 6.16,
	"VersionName": "6.16.3",
	"FriendlyName": "DNACalib Plugin",
	"Description": "DNA Calibration tool plugin",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "DNACalibLib",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "DNACalibLibTest",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "DNACalibModule",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "RigLogic",
			"Enabled": true
		}
	]
}