// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DNACalibLib : ModuleRules
	{
		public DNACalibLib(ReadOnlyTargetRules Target) : base(Target)
		{
			IWYUSupport = IWYUSupport.None;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
						"Core",
						"CoreUObject",
						"Engine",
						"RigLogicLib"
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PublicDependencyModuleNames.Add("UnrealEd");
			}

			Type = ModuleType.CPlusPlus;

			if (Target.LinkType != TargetLinkType.Monolithic)
			{
				PrivateDefinitions.Add("DNAC_BUILD_SHARED=1");
				PublicDefinitions.Add("DNAC_SHARED=1");
			}
		}
	}
}
