// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef DNACALIB_MODULE_DISCARD

#include "DNACalibLib.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FDNACalibLib"

DEFINE_LOG_CATEGORY(LogDNACalibLib);

void FDNACalibLib::StartupModule()
{
}

void FDNACalibLib::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FDNACalibLib, DNACalibLib)

#endif  // DNACALIB_MODULE_DISCARD

