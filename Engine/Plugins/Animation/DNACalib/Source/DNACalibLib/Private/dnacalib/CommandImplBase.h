// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

template<class TCommand>
class CommandImplBase {
protected:
    explicit CommandImplBase(MemoryResource* memRes_) :
        memRes{memRes_} {
    }

public:
    static TCommand* create(MemoryResource* memRes) {
        PolyAllocator<TCommand> alloc{memRes};
        return alloc.newObject(memRes);
    }

    static void destroy(TCommand* instance) {
        PolyAllocator<TCommand> alloc{instance->getMemoryResource()};
        alloc.deleteObject(instance);
    }

    MemoryResource* getMemoryResource() {
        return memRes;
    }

private:
    MemoryResource* memRes;
};

}  // namespace dnac
