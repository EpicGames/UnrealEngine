// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/types/Aliases.h"

#include <pma/PolyAllocator.h>
#include <pma/TypeDefs.h>
#include <pma/resources/AlignedMemoryResource.h>
#include <pma/resources/ArenaMemoryResource.h>
#include <pma/resources/DefaultMemoryResource.h>
#include <status/Provider.h>
#include <tdm/TDM.h>
#include <terse/types/Blob.h>
#include <terse/types/DynArray.h>

namespace dnac {

using namespace tdm;
using namespace pma;

template<typename T>
using AlignedAllocator = PolyAllocator<T, 64ul, AlignedMemoryResource>;

template<typename T>
using Blob = terse::Blob<T, PolyAllocator<T>>;

template<typename T>
using DynArray = terse::DynArray<T, PolyAllocator<T>>;

template<typename T>
using AlignedDynArray = terse::DynArray<T, AlignedAllocator<T>>;

}  // namespace dnac
