// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/CalculateMeshLowerLODsCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/commands/CalculateMeshLowerLODsCommandImpl.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"
#include "dnacalib/types/UVBarycentricMapping.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstddef>
#include <cstdio>
#include <iterator>
#include <limits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace dnac {

class CalculateMeshLowerLODsCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        meshIndex{} {
    }

    void setMeshIndex(std::uint16_t meshIndex_) {
        meshIndex = meshIndex_;
    }

    void run(DNACalibDNAReaderImpl* output) {
        auto faceGetter = std::bind(&dna::Reader::getFaceVertexLayoutIndices, output, meshIndex, std::placeholders::_1);
        const auto layoutPositions = output->getVertexLayoutPositionIndices(meshIndex);
        const auto layoutTexCoords = output->getVertexLayoutTextureCoordinateIndices(meshIndex);
        const auto origMappingUs = output->getVertexTextureCoordinateUs(meshIndex);
        const auto mappingVs = output->getVertexTextureCoordinateVs(meshIndex);
        const auto mappingUs = deduplicateTextureCoordinates(origMappingUs, mappingVs);
        const auto faceCount = output->getFaceCount(meshIndex);

        UVBarycentricMapping
            mapping{faceGetter, layoutPositions, layoutTexCoords, mappingUs, mappingVs, faceCount, getMemoryResource()};

        auto srcMeshXs = output->getVertexPositionXs(meshIndex);
        auto srcMeshYs = output->getVertexPositionYs(meshIndex);
        auto srcMeshZs = output->getVertexPositionZs(meshIndex);

        const auto getSrcVertex = [srcMeshXs, srcMeshYs, srcMeshZs](std::uint32_t positionIndex) {
            return fvec3{srcMeshXs[positionIndex], srcMeshYs[positionIndex], srcMeshZs[positionIndex]};
        };

        for (std::uint16_t mi : findIndicesOfMeshLowerLODs(output)) {
            const auto vertexLayoutPositionIndices = output->getVertexLayoutPositionIndices(mi);
            const auto vertexLayoutTextureCoordinateIndices = output->getVertexLayoutTextureCoordinateIndices(mi);
            const auto vs = output->getVertexTextureCoordinateVs(mi);
            const auto us = deduplicateTextureCoordinates(output->getVertexTextureCoordinateUs(mi), vs);
            const std::uint32_t positionCount = output->getVertexPositionCount(mi);
            RawVector3Vector destVertexPositions{positionCount, {}, getMemoryResource()};
            // As there can be multiple VertexLayout per each VertexPosition we will use arithmetic mean value.
            Vector<std::uint8_t> vertexLayoutsPerPosition{positionCount, {}, getMemoryResource()};

            for (std::uint32_t vli = 0u; vli < vertexLayoutPositionIndices.size(); ++vli) {
                std::uint32_t uvIndex = vertexLayoutTextureCoordinateIndices[vli];
                const fvec2 uvs = {us[uvIndex], vs[uvIndex]};
                const auto weightsIndicesPair = mapping.getBarycentric(uvs);
                fvec3 barycentric = std::get<0>(weightsIndicesPair);
                auto srcVtxIndices = std::get<1>(weightsIndicesPair);

                if (srcVtxIndices.size() == 0) {
                    // We didn't hit any triangle. We aim to identify the nearest face to this UV, ensuring
                    // that the selected face has an intersection with at least one of the adjacent faces of the vertex we are
                    // projecting.
                    float minDistance = std::numeric_limits<float>::max();
                    std::uint32_t sourceTriangleIndex = std::numeric_limits<std::uint32_t>::max();
                    // First we find all of the faces that are adjacent to this vertex
                    for (std::uint32_t fi = 0u; fi < output->getFaceCount(mi); fi++) {
                        const auto face = output->getFaceVertexLayoutIndices(mi, fi);
                        if (std::find(face.begin(), face.end(), vli) == face.end()) {
                            continue;
                        }

                        // Gather all vertex UVs from this face and create a bounding box from it
                        Vector<fvec2> UVs{getMemoryResource()};
                        for (const auto vertexLayoutIndex : face) {
                            uvIndex = vertexLayoutTextureCoordinateIndices[vertexLayoutIndex];
                            UVs.emplace_back(us[uvIndex], vs[uvIndex]);
                        }
                        const BoundingBox faceBoundingBox{UVs};

                        // Find the closest triangle that has intersection with this face
                        auto bBoxes = mapping.getBoundingBoxes();
                        for (std::uint32_t bi = 0u; bi < bBoxes.size(); bi++) {
                            const auto& bBox = bBoxes[bi];
                            if (bBox.overlaps(faceBoundingBox)) {
                                const float distance = bBox.distance(uvs);
                                if (distance < minDistance) {
                                    minDistance = distance;
                                    sourceTriangleIndex = bi;
                                }
                            }
                        }
                    }

                    if (sourceTriangleIndex != std::numeric_limits<std::uint32_t>::max()) {
                        barycentric = mapping.getTriangle(sourceTriangleIndex).getBarycentricCoords(uvs);
                        srcVtxIndices = mapping.getTrianglePositionIndices(sourceTriangleIndex);
                    } else {
                        assert(false && "Could not map a vertex. It is not within a face of higher lod.");
                        continue;
                    }
                }
                const fvec3 src = getSrcVertex(srcVtxIndices[0]) * barycentric[0] +
                                  getSrcVertex(srcVtxIndices[1]) * barycentric[1] +
                                  getSrcVertex(srcVtxIndices[2]) * barycentric[2];

                const uint32_t positionIndex = vertexLayoutPositionIndices[vli];
                float& destX = destVertexPositions.xs[positionIndex];
                float& destY = destVertexPositions.ys[positionIndex];
                float& destZ = destVertexPositions.zs[positionIndex];

                const auto vtxLayoutCount = ++vertexLayoutsPerPosition[positionIndex];
                // We require mean average, more than one vertexLayout for this vertex position
                const auto lastDenominator = static_cast<float>(vtxLayoutCount - 1u);
                const auto newDenominator = static_cast<float>(vtxLayoutCount);
                destX = (destX * lastDenominator + src[0]) / newDenominator;
                destY = (destY * lastDenominator + src[1]) / newDenominator;
                destZ = (destZ * lastDenominator + src[2]) / newDenominator;
            }
            output->setVertexPositions(mi, std::move(destVertexPositions));
        }
    }

private:
    /**
     * @brief Get the Mesh Name without postfix " _lodX_mesh"
     */
    static StringView getMeshName(DNACalibDNAReaderImpl* output, std::uint16_t mi) {
        const auto meshName = output->getMeshName(mi);
        const auto underscoreIter = std::find(meshName.begin(), meshName.end(), '_');
        assert(underscoreIter != meshName.end() && "Mesh naming does not follow convention.");

        auto length = static_cast<size_t>(std::distance(meshName.begin(), underscoreIter));
        return {meshName.data(), length};
    }

    Vector<std::uint16_t> findIndicesOfMeshLowerLODs(DNACalibDNAReaderImpl* output) {
        Vector<std::uint16_t> lowerLODIndices{getMemoryResource()};
        bool isLowerLOD = false;
        auto meshName = getMeshName(output, meshIndex);
        for (std::uint16_t lodIndex = 0u; lodIndex < output->getLODCount(); ++lodIndex) {
            auto lodMeshIndices = output->getMeshIndicesForLOD(lodIndex);
            if (isLowerLOD) {
                for (std::uint16_t mi : lodMeshIndices) {
                    if (meshName == getMeshName(output, mi)) {
                        lowerLODIndices.push_back(mi);
                        break;
                    }
                }
            } else {
                isLowerLOD = std::find(lodMeshIndices.begin(), lodMeshIndices.end(), meshIndex) != lodMeshIndices.end();
            }
        }
        return lowerLODIndices;
    }

    Vector<float> deduplicateTextureCoordinates(ConstArrayView<float> us, ConstArrayView<float> vs) {
        Vector<float> usCopy{us.begin(), us.end(), getMemoryResource()};
        if (isUVMapOverlapping(us, vs)) {
            // The offset function will not modify those given arrays for which the specified offset is 0.0
            // So const_cast-ing here is just to satisfy the compiler, not for modifying the data sneakily.
            offsetOverlappingUVMapRegion(usCopy, {const_cast<float*>(vs.data()), vs.size()}, 1.0f, 0.0f);
        }
        return usCopy;
    }

private:
    std::uint16_t meshIndex;
};

CalculateMeshLowerLODsCommand::CalculateMeshLowerLODsCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

CalculateMeshLowerLODsCommand::CalculateMeshLowerLODsCommand(std::uint16_t meshIndex, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndex(meshIndex);
}

CalculateMeshLowerLODsCommand::~CalculateMeshLowerLODsCommand() = default;
CalculateMeshLowerLODsCommand::CalculateMeshLowerLODsCommand(CalculateMeshLowerLODsCommand&&) = default;
CalculateMeshLowerLODsCommand& CalculateMeshLowerLODsCommand::operator=(CalculateMeshLowerLODsCommand&&) = default;

void CalculateMeshLowerLODsCommand::setMeshIndex(std::uint16_t meshIndex) {
    pImpl->setMeshIndex(meshIndex);
}

void CalculateMeshLowerLODsCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
