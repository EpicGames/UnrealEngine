// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/CalculateMeshLowerLODsCommandImpl.h"

#include <algorithm>
#include <cmath>
#include <cstddef>

namespace dnac {

inline bool near(float a, float b, float threshold) {
    return std::fabs(a - b) < threshold;
}

bool isUVMapOverlapping(ConstArrayView<float> us,
                        ConstArrayView<float> vs,
                        std::size_t overlapCountThreshold,
                        float uvCompareThreshold) {
    // Quick heuristic to check if the UV is really mirrored into the upper half of the array,
    // if first N matches, it will be considered a total match and deduplication should proceed
    assert(us.size() == vs.size());

    if ((us.size() % 2ul) != 0ul) {
        return false;
    }

    const std::size_t half = (us.size() / 2ul);
    for (std::size_t i = {}; i < std::min(half, overlapCountThreshold); ++i) {
        bool found = false;
        for (std::size_t j = half; j < us.size(); ++j) {
            if (near(us[i], us[j], uvCompareThreshold) && near(vs[i], vs[j], uvCompareThreshold)) {
                found = true;
                break;
            }
        }
        if (!found) {
            return false;
        }
    }
    return true;
}

void
offsetOverlappingUVMapRegion(ArrayView<float> us, ArrayView<float> vs, float uOffset, float vOffset, float uvCompareThreshold) {
    assert(us.size() == vs.size());
    const std::size_t half = (us.size() / 2ul);
    for (std::size_t i = {}; i < half; ++i) {
        for (std::size_t j = half; j < us.size(); ++j) {
            if (near(us[i], us[j], uvCompareThreshold) && near(vs[i], vs[j], uvCompareThreshold)) {
                if (uOffset != 0.0f) {
                    us[i] += uOffset;
                }
                if (vOffset != 0.0f) {
                    vs[i] += vOffset;
                }
                break;
            }
        }
    }
}

}  // namespace dnac
