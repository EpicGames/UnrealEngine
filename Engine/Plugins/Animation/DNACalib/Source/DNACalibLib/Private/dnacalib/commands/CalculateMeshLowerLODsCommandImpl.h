// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/types/Aliases.h"

namespace dnac {

bool isUVMapOverlapping(ConstArrayView<float> us,
                        ConstArrayView<float> vs,
                        std::size_t overlapCountThreshold = 10ul,
                        float uvCompareThreshold = 0.0002f);
void offsetOverlappingUVMapRegion(ArrayView<float> us,
                                  ArrayView<float> vs,
                                  float uOffset = 1.0f,
                                  float vOffset = 0.0f,
                                  float uvCompareThreshold = 0.0002f);

}  // namespace dnac
