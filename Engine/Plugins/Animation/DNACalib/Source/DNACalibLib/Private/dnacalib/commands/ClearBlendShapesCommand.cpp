// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/ClearBlendShapesCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class ClearBlendShapesCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_} {
    }

    void run(DNACalibDNAReaderImpl* output) {
        output->clearBlendShapeChannelNames();
        output->clearBlendShapeChannelIndices();
        output->clearLODBlendShapeChannelMappings();
        output->clearMeshBlendShapeChannelMappings();
        for (std::uint16_t i = 0; i < output->getMeshCount(); ++i) {
            output->clearBlendShapeTargets(i);
        }
        Vector<std::uint16_t> lods{output->getLODCount(), 0u, getMemoryResource()};
        output->setBlendShapeChannelLODs(lods.data(), static_cast<std::uint16_t>(lods.size()));
        output->setBlendShapeChannelInputIndices(nullptr, 0);
        output->setBlendShapeChannelOutputIndices(nullptr, 0);
    }
};

ClearBlendShapesCommand::ClearBlendShapesCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

ClearBlendShapesCommand::~ClearBlendShapesCommand() = default;
ClearBlendShapesCommand::ClearBlendShapesCommand(ClearBlendShapesCommand&&) = default;
ClearBlendShapesCommand& ClearBlendShapesCommand::operator=(ClearBlendShapesCommand&&) = default;

void ClearBlendShapesCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
