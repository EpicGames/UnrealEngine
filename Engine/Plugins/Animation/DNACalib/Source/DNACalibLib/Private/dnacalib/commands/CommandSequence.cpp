// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/CommandSequence.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/TypeDefs.h"

namespace dnac {

class CommandSequence::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        commands{memRes_} {
    }

    void run(DNACalibDNAReader* output) {
        for (auto& cmd : commands) {
            cmd->run(output);
        }
    }

    void add(Command* command) {
        commands.push_back(command);
    }

    void remove(Command* command) {
        auto it = std::find(commands.begin(), commands.end(), command);
        if (it != commands.end()) {
            commands.erase(it);
        }
    }

    bool contains(Command* command) const {
        return (std::find(commands.begin(), commands.end(), command) != commands.end());
    }

    std::size_t size() const {
        return commands.size();
    }

private:
    Vector<Command*> commands;
};

CommandSequence::CommandSequence(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

CommandSequence::~CommandSequence() = default;
CommandSequence::CommandSequence(CommandSequence&&) = default;
CommandSequence& CommandSequence::operator=(CommandSequence&&) = default;

void CommandSequence::run(DNACalibDNAReader* output) {
    pImpl->run(output);
}

void CommandSequence::add(Command* command) {
    pImpl->add(command);
}

void CommandSequence::add(ArrayView<Command> commands) {
    for (auto& cmd : commands) {
        pImpl->add(&cmd);
    }
}

void CommandSequence::remove(Command* command) {
    pImpl->remove(command);
}

void CommandSequence::remove(ArrayView<Command> commands) {
    for (auto& cmd : commands) {
        pImpl->remove(&cmd);
    }
}

bool CommandSequence::contains(Command* command) const {
    return pImpl->contains(command);
}

std::size_t CommandSequence::size() const {
    return pImpl->size();
}

}  // namespace dnac
