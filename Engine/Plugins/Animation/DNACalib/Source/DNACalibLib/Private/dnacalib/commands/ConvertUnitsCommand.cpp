// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/ConvertUnitsCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class ConvertUnitsCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_} {
    }

    void run(DNACalibDNAReaderImpl* output) {
        output->convertToTranslationUnit(translationUnit);
        output->convertToRotationUnit(rotationUnit);
    }

    void setTranslationUnit(TranslationUnit translationUnit_) {
        translationUnit = translationUnit_;
    }

    void setRotationUnit(RotationUnit rotationUnit_) {
        rotationUnit = rotationUnit_;
    }

private:
    TranslationUnit translationUnit;
    RotationUnit rotationUnit;
};

ConvertUnitsCommand::ConvertUnitsCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

ConvertUnitsCommand::ConvertUnitsCommand(TranslationUnit translationUnit, RotationUnit rotationUnit, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
    pImpl->setTranslationUnit(translationUnit);
    pImpl->setRotationUnit(rotationUnit);
}

ConvertUnitsCommand::~ConvertUnitsCommand() = default;
ConvertUnitsCommand::ConvertUnitsCommand(ConvertUnitsCommand&&) = default;
ConvertUnitsCommand& ConvertUnitsCommand::operator=(ConvertUnitsCommand&&) = default;

void ConvertUnitsCommand::setTranslationUnit(TranslationUnit translationUnit) {
    pImpl->setTranslationUnit(translationUnit);
}

void ConvertUnitsCommand::setRotationUnit(RotationUnit rotationUnit) {
    pImpl->setRotationUnit(rotationUnit);
}

void ConvertUnitsCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
