// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/PruneBlendShapeTargetsCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class PruneBlendShapeTargetsCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        threshold{} {
    }

    void setThreshold(float threshold_) {
        threshold = threshold_;
    }

    void run(DNACalibDNAReaderImpl* output) {
        output->pruneBlendShapeTargets(threshold);
    }

private:
    float threshold;
};

PruneBlendShapeTargetsCommand::PruneBlendShapeTargetsCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

PruneBlendShapeTargetsCommand::PruneBlendShapeTargetsCommand(float threshold, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setThreshold(threshold);
}

PruneBlendShapeTargetsCommand::~PruneBlendShapeTargetsCommand() = default;
PruneBlendShapeTargetsCommand::PruneBlendShapeTargetsCommand(PruneBlendShapeTargetsCommand&&) = default;
PruneBlendShapeTargetsCommand& PruneBlendShapeTargetsCommand::operator=(PruneBlendShapeTargetsCommand&&) = default;

void PruneBlendShapeTargetsCommand::setThreshold(float threshold) {
    pImpl->setThreshold(threshold);
}

void PruneBlendShapeTargetsCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
