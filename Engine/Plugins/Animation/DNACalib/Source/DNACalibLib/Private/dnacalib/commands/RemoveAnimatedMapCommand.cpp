// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/RemoveAnimatedMapCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class RemoveAnimatedMapCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        animatedMapIndices{memRes_} {
    }

    void setAnimatedMapIndex(std::uint16_t animatedMapIndex_) {
        animatedMapIndices.resize(1);
        animatedMapIndices[0] = animatedMapIndex_;
    }

    void setAnimatedMapIndices(ConstArrayView<std::uint16_t> animatedMapIndices_) {
        animatedMapIndices.assign(animatedMapIndices_.begin(), animatedMapIndices_.end());
    }

    void run(DNACalibDNAReaderImpl* output) {
        output->removeAnimatedMaps(ConstArrayView<std::uint16_t>{animatedMapIndices});
    }

private:
    Vector<std::uint16_t> animatedMapIndices;
};

RemoveAnimatedMapCommand::RemoveAnimatedMapCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

RemoveAnimatedMapCommand::RemoveAnimatedMapCommand(std::uint16_t animatedMapIndex, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setAnimatedMapIndex(animatedMapIndex);
}

RemoveAnimatedMapCommand::RemoveAnimatedMapCommand(ConstArrayView<std::uint16_t> animatedMapIndices, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setAnimatedMapIndices(animatedMapIndices);
}

RemoveAnimatedMapCommand::~RemoveAnimatedMapCommand() = default;
RemoveAnimatedMapCommand::RemoveAnimatedMapCommand(RemoveAnimatedMapCommand&&) = default;
RemoveAnimatedMapCommand& RemoveAnimatedMapCommand::operator=(RemoveAnimatedMapCommand&&) = default;

void RemoveAnimatedMapCommand::setAnimatedMapIndex(std::uint16_t animatedMapIndex) {
    pImpl->setAnimatedMapIndex(animatedMapIndex);
}

void RemoveAnimatedMapCommand::setAnimatedMapIndices(ConstArrayView<std::uint16_t> animatedMapIndices) {
    pImpl->setAnimatedMapIndices(animatedMapIndices);
}

void RemoveAnimatedMapCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
