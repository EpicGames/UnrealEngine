// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/RemoveBlendShapeCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class RemoveBlendShapeCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        blendShapeIndices{memRes_} {
    }

    void setBlendShapeIndex(std::uint16_t blendShapeIndex_) {
        blendShapeIndices.resize(1);
        blendShapeIndices[0] = blendShapeIndex_;
    }

    void setBlendShapeIndices(ConstArrayView<std::uint16_t> blendShapeIndices_) {
        blendShapeIndices.assign(blendShapeIndices_.begin(), blendShapeIndices_.end());
    }

    void run(DNACalibDNAReaderImpl* output) {
        output->removeBlendShapes(ConstArrayView<std::uint16_t>{blendShapeIndices});
    }

private:
    Vector<std::uint16_t> blendShapeIndices;
};

RemoveBlendShapeCommand::RemoveBlendShapeCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

RemoveBlendShapeCommand::RemoveBlendShapeCommand(std::uint16_t blendShapeIndex, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setBlendShapeIndex(blendShapeIndex);
}

RemoveBlendShapeCommand::RemoveBlendShapeCommand(ConstArrayView<std::uint16_t> blendShapeIndices, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setBlendShapeIndices(blendShapeIndices);
}

RemoveBlendShapeCommand::~RemoveBlendShapeCommand() = default;
RemoveBlendShapeCommand::RemoveBlendShapeCommand(RemoveBlendShapeCommand&&) = default;
RemoveBlendShapeCommand& RemoveBlendShapeCommand::operator=(RemoveBlendShapeCommand&&) = default;

void RemoveBlendShapeCommand::setBlendShapeIndex(std::uint16_t blendShapeIndex) {
    pImpl->setBlendShapeIndex(blendShapeIndex);
}

void RemoveBlendShapeCommand::setBlendShapeIndices(ConstArrayView<std::uint16_t> blendShapeIndices) {
    pImpl->setBlendShapeIndices(blendShapeIndices);
}

void RemoveBlendShapeCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
