// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/RemoveJointAnimationCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class RemoveJointAnimationCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        jointIndices{memRes_} {
    }

    void setJointIndex(std::uint16_t jointIndex_) {
        jointIndices.resize(1);
        jointIndices[0] = jointIndex_;
    }

    void setJointIndices(ConstArrayView<std::uint16_t> jointIndices_) {
        jointIndices.assign(jointIndices_.begin(), jointIndices_.end());
    }

    void run(DNACalibDNAReaderImpl* output) {
        output->removeJointAnimations(ConstArrayView<std::uint16_t>{jointIndices});
    }

private:
    Vector<std::uint16_t> jointIndices;
};

RemoveJointAnimationCommand::RemoveJointAnimationCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

RemoveJointAnimationCommand::RemoveJointAnimationCommand(std::uint16_t jointIndex, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setJointIndex(jointIndex);
}

RemoveJointAnimationCommand::RemoveJointAnimationCommand(ConstArrayView<std::uint16_t> jointIndices, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setJointIndices(jointIndices);
}

RemoveJointAnimationCommand::~RemoveJointAnimationCommand() = default;
RemoveJointAnimationCommand::RemoveJointAnimationCommand(RemoveJointAnimationCommand&&) = default;
RemoveJointAnimationCommand& RemoveJointAnimationCommand::operator=(RemoveJointAnimationCommand&&) = default;

void RemoveJointAnimationCommand::setJointIndex(std::uint16_t jointIndex) {
    pImpl->setJointIndex(jointIndex);
}

void RemoveJointAnimationCommand::setJointIndices(ConstArrayView<std::uint16_t> jointIndices) {
    pImpl->setJointIndices(jointIndices);
}

void RemoveJointAnimationCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
