// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/RemoveJointCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class RemoveJointCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        jointIndices{memRes_} {
    }

    void setJointIndex(std::uint16_t jointIndex_) {
        jointIndices.resize(1);
        jointIndices[0] = jointIndex_;
    }

    void setJointIndices(ConstArrayView<std::uint16_t> jointIndices_) {
        jointIndices.assign(jointIndices_.begin(), jointIndices_.end());
    }

    void run(DNACalibDNAReaderImpl* output) {
        output->removeJoints(ConstArrayView<std::uint16_t>{jointIndices});
    }

private:
    Vector<std::uint16_t> jointIndices;
};

RemoveJointCommand::RemoveJointCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

RemoveJointCommand::RemoveJointCommand(std::uint16_t jointIndex, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setJointIndex(jointIndex);
}

RemoveJointCommand::RemoveJointCommand(ConstArrayView<std::uint16_t> jointIndices, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setJointIndices(jointIndices);
}

RemoveJointCommand::~RemoveJointCommand() = default;
RemoveJointCommand::RemoveJointCommand(RemoveJointCommand&&) = default;
RemoveJointCommand& RemoveJointCommand::operator=(RemoveJointCommand&&) = default;

void RemoveJointCommand::setJointIndex(std::uint16_t jointIndex) {
    pImpl->setJointIndex(jointIndex);
}

void RemoveJointCommand::setJointIndices(ConstArrayView<std::uint16_t> jointIndices) {
    pImpl->setJointIndices(jointIndices);
}

void RemoveJointCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
