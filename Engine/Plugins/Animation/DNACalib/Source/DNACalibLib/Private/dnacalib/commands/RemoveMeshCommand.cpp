// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/RemoveMeshCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class RemoveMeshCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        meshIndices{memRes_} {
    }

    void setMeshIndex(std::uint16_t meshIndex_) {
        meshIndices.resize(1);
        meshIndices[0] = meshIndex_;
    }

    void setMeshIndices(ConstArrayView<std::uint16_t> meshIndices_) {
        meshIndices.assign(meshIndices_.begin(), meshIndices_.end());
    }

    void run(DNACalibDNAReaderImpl* output) {
        output->removeMeshes(ConstArrayView<std::uint16_t>{meshIndices});
    }

private:
    Vector<std::uint16_t> meshIndices;
};

RemoveMeshCommand::RemoveMeshCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

RemoveMeshCommand::RemoveMeshCommand(std::uint16_t meshIndex, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndex(meshIndex);
}

RemoveMeshCommand::RemoveMeshCommand(ConstArrayView<std::uint16_t> meshIndices, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndices(meshIndices);
}

RemoveMeshCommand::~RemoveMeshCommand() = default;
RemoveMeshCommand::RemoveMeshCommand(RemoveMeshCommand&&) = default;
RemoveMeshCommand& RemoveMeshCommand::operator=(RemoveMeshCommand&&) = default;

void RemoveMeshCommand::setMeshIndex(std::uint16_t meshIndex) {
    pImpl->setMeshIndex(meshIndex);
}

void RemoveMeshCommand::setMeshIndices(ConstArrayView<std::uint16_t> meshIndices) {
    pImpl->setMeshIndices(meshIndices);
}

void RemoveMeshCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
