// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/RenameAnimatedMapCommand.h"

#include "dnacalib/commands/RenameResourceCommand.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

#include <cstring>

namespace dnac {

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wweak-vtables"
#endif
class RenameAnimatedMapCommand::Impl : public RenameResourceCommand<Impl> {
private:
    using Super = RenameResourceCommand<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_} {
    }

private:
    std::uint16_t getNameCount(const dna::Reader* input) const override {
        return input->getAnimatedMapCount();
    }

    StringView getNameByIndex(const dna::Reader* input, std::uint16_t index_) const override {
        return input->getAnimatedMapName(index_);
    }

    void setNameByIndex(dna::Writer* output, std::uint16_t index_, const char* name) override {
        output->setAnimatedMapName(index_, name);
    }
};
#ifdef __clang__
    #pragma clang diagnostic pop
#endif

RenameAnimatedMapCommand::RenameAnimatedMapCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

RenameAnimatedMapCommand::RenameAnimatedMapCommand(std::uint16_t animatedMapIndex, const char* newName, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setName(animatedMapIndex, newName);
}

RenameAnimatedMapCommand::RenameAnimatedMapCommand(const char* oldName, const char* newName, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setName(oldName, newName);
}

RenameAnimatedMapCommand::~RenameAnimatedMapCommand() = default;
RenameAnimatedMapCommand::RenameAnimatedMapCommand(RenameAnimatedMapCommand&&) = default;
RenameAnimatedMapCommand& RenameAnimatedMapCommand::operator=(RenameAnimatedMapCommand&&) = default;

void RenameAnimatedMapCommand::setName(std::uint16_t animatedMapIndex, const char* newName) {
    pImpl->setName(animatedMapIndex, newName);
}

void RenameAnimatedMapCommand::setName(const char* oldName, const char* newName) {
    pImpl->setName(oldName, newName);
}

void RenameAnimatedMapCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
