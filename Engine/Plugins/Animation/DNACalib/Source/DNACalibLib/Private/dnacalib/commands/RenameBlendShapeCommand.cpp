// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/RenameBlendShapeCommand.h"

#include "dnacalib/commands/RenameResourceCommand.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

#include <cstring>

namespace dnac {

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wweak-vtables"
#endif
class RenameBlendShapeCommand::Impl : public RenameResourceCommand<Impl> {
private:
    using Super = RenameResourceCommand<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_} {
    }

private:
    std::uint16_t getNameCount(const dna::Reader* input) const override {
        return input->getBlendShapeChannelCount();
    }

    StringView getNameByIndex(const dna::Reader* input, std::uint16_t index_) const override {
        return input->getBlendShapeChannelName(index_);
    }

    void setNameByIndex(dna::Writer* output, std::uint16_t index_, const char* name) override {
        output->setBlendShapeChannelName(index_, name);
    }
};
#ifdef __clang__
    #pragma clang diagnostic pop
#endif

RenameBlendShapeCommand::RenameBlendShapeCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

RenameBlendShapeCommand::RenameBlendShapeCommand(std::uint16_t blendShapeIndex, const char* newName, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setName(blendShapeIndex, newName);
}

RenameBlendShapeCommand::RenameBlendShapeCommand(const char* oldName, const char* newName, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setName(oldName, newName);
}

RenameBlendShapeCommand::~RenameBlendShapeCommand() = default;
RenameBlendShapeCommand::RenameBlendShapeCommand(RenameBlendShapeCommand&&) = default;
RenameBlendShapeCommand& RenameBlendShapeCommand::operator=(RenameBlendShapeCommand&&) = default;

void RenameBlendShapeCommand::setName(std::uint16_t blendShapeIndex, const char* newName) {
    pImpl->setName(blendShapeIndex, newName);
}

void RenameBlendShapeCommand::setName(const char* oldName, const char* newName) {
    pImpl->setName(oldName, newName);
}

void RenameBlendShapeCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
