// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/RenameJointCommand.h"

#include "dnacalib/commands/RenameResourceCommand.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

#include <cstring>

namespace dnac {

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wweak-vtables"
#endif
class RenameJointCommand::Impl : public RenameResourceCommand<Impl> {
private:
    using Super = RenameResourceCommand<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_} {
    }

private:
    std::uint16_t getNameCount(const dna::Reader* input) const override {
        return input->getJointCount();
    }

    StringView getNameByIndex(const dna::Reader* input, std::uint16_t index_) const override {
        return input->getJointName(index_);
    }

    void setNameByIndex(dna::Writer* output, std::uint16_t index_, const char* name) override {
        output->setJointName(index_, name);
    }
};
#ifdef __clang__
    #pragma clang diagnostic pop
#endif

RenameJointCommand::RenameJointCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

RenameJointCommand::RenameJointCommand(std::uint16_t jointIndex, const char* newName, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setName(jointIndex, newName);
}

RenameJointCommand::RenameJointCommand(const char* oldName, const char* newName, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setName(oldName, newName);
}

RenameJointCommand::~RenameJointCommand() = default;
RenameJointCommand::RenameJointCommand(RenameJointCommand&&) = default;
RenameJointCommand& RenameJointCommand::operator=(RenameJointCommand&&) = default;

void RenameJointCommand::setName(std::uint16_t jointIndex, const char* newName) {
    pImpl->setName(jointIndex, newName);
}

void RenameJointCommand::setName(const char* oldName, const char* newName) {
    pImpl->setName(oldName, newName);
}

void RenameJointCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
