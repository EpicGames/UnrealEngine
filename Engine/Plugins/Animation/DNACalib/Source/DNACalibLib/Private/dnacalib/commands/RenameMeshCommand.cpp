// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/RenameMeshCommand.h"

#include "dnacalib/commands/RenameResourceCommand.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

#include <cstring>

namespace dnac {

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wweak-vtables"
#endif
class RenameMeshCommand::Impl : public RenameResourceCommand<Impl> {
private:
    using Super = RenameResourceCommand<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_} {
    }

private:
    std::uint16_t getNameCount(const dna::Reader* input) const override {
        return input->getMeshCount();
    }

    StringView getNameByIndex(const dna::Reader* input, std::uint16_t index_) const override {
        return input->getMeshName(index_);
    }

    void setNameByIndex(dna::Writer* output, std::uint16_t index_, const char* name) override {
        output->setMeshName(index_, name);
    }
};
#ifdef __clang__
    #pragma clang diagnostic pop
#endif

RenameMeshCommand::RenameMeshCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

RenameMeshCommand::RenameMeshCommand(std::uint16_t meshIndex, const char* newName, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setName(meshIndex, newName);
}

RenameMeshCommand::RenameMeshCommand(const char* oldName, const char* newName, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setName(oldName, newName);
}

RenameMeshCommand::~RenameMeshCommand() = default;
RenameMeshCommand::RenameMeshCommand(RenameMeshCommand&&) = default;
RenameMeshCommand& RenameMeshCommand::operator=(RenameMeshCommand&&) = default;

void RenameMeshCommand::setName(std::uint16_t meshIndex, const char* newName) {
    pImpl->setName(meshIndex, newName);
}

void RenameMeshCommand::setName(const char* oldName, const char* newName) {
    pImpl->setName(oldName, newName);
}

void RenameMeshCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
