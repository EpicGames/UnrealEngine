// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"

#include <cstdint>
#include <cstring>

namespace dnac {

template<class TDerived>
class RenameResourceCommand : public CommandImplBase<TDerived> {
private:
    using Super = CommandImplBase<TDerived>;

    enum class Configuration {
        Unconfigured,
        SearchAndRename,
        RenameByIndex
    };

public:
    explicit RenameResourceCommand(MemoryResource* memRes_) :
        Super{memRes_},
        oldName{memRes_},
        newName{memRes_},
        index{},
        config{Configuration::Unconfigured} {
    }

    virtual ~RenameResourceCommand() = default;

    RenameResourceCommand(const RenameResourceCommand&) = default;
    RenameResourceCommand& operator=(const RenameResourceCommand&) = default;

    RenameResourceCommand(RenameResourceCommand&&) = default;
    RenameResourceCommand& operator=(RenameResourceCommand&&) = default;

    void setName(std::uint16_t index_, const char* newName_) {
        index = index_;
        newName = newName_;
        config = Configuration::RenameByIndex;
    }

    void setName(const char* oldName_, const char* newName_) {
        oldName = oldName_;
        newName = newName_;
        config = Configuration::SearchAndRename;
    }

    void run(DNACalibDNAReaderImpl* output) {
        if (config == Configuration::RenameByIndex) {
            rename(output);
        } else if (config == Configuration::SearchAndRename) {
            searchAndRename(output);
        }
    }

private:
    void searchAndRename(DNACalibDNAReaderImpl* output) {
        for (std::uint16_t i = 0u; i < getNameCount(output); ++i) {
            const auto name = getNameByIndex(output, i);
            if (name == oldName) {
                setNameByIndex(output, i, newName.c_str());
                return;
            }
        }
    }

    void rename(DNACalibDNAReaderImpl* output) {
        setNameByIndex(output, index, newName.c_str());
    }

    virtual std::uint16_t getNameCount(const dna::Reader* input) const = 0;
    virtual StringView getNameByIndex(const dna::Reader* input, std::uint16_t index_) const = 0;
    virtual void setNameByIndex(dna::Writer* output, std::uint16_t index_, const char* name) = 0;

private:
    String<char> oldName;
    String<char> newName;
    std::uint16_t index;
    Configuration config;
};

}  // namespace dnac
