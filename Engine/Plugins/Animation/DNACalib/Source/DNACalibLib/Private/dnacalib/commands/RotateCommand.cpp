// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/RotateCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"
#include "dnacalib/utils/Algorithm.h"

#include <tdm/TDM.h>

namespace dnac {

class RotateCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        degrees{},
        origin{} {
    }

    void setRotation(Vector3 degrees_) {
        degrees = degrees_;
    }

    void setOrigin(Vector3 origin_) {
        origin = origin_;
    }

    void run(DNACalibDNAReaderImpl* output) {
        if (degrees != Vector3{}) {
            rotateNeutralJoints(output);
            rotateVertexPositions(output);
            rotateBlendShapeTargetDeltas(output);
        }
    }

private:
    tdm::mat4<float> getRotationTransformationMatrix(tdm::rot_seq rotationSequence, const tdm::rot_sign& rotationSigns) const {
        const auto inverseTranslationMatrix = tdm::translate(tdm::fvec3{-origin.x, -origin.y, -origin.z});
        const auto rotationMatrix = tdm::rotate(tdm::frad{tdm::fdeg{degrees.x}},
                                                tdm::frad{tdm::fdeg{degrees.y}},
                                                tdm::frad{tdm::fdeg{degrees.z}},
                                                rotationSequence,
                                                rotationSigns);
        const auto translationMatrix = tdm::translate(tdm::fvec3{origin.x, origin.y, origin.z});
        return inverseTranslationMatrix * rotationMatrix * translationMatrix;
    }

    void rotateNeutralJoints(DNACalibDNAReaderImpl* output) {
        const auto rotationUnit = output->getRotationUnit();
        const auto rotationSequence = output->getRotationSequence();
        const auto rotationSigns = output->getRotationSign();
        const auto rotationMatrix = getRotationTransformationMatrix(rotationSequence, rotationSigns);
        for (std::uint16_t jointIndex = 0u; jointIndex < output->getJointCount(); ++jointIndex) {
            const auto parentIndex = output->getJointParentIndex(jointIndex);
            // Only root joints are rotated
            if (jointIndex == parentIndex) {
                const auto jointNeutralRotation = output->getNeutralJointRotation(jointIndex);
                const auto jointNeutralTranslation = output->getNeutralJointTranslation(jointIndex);
                const auto angle2rad = [rotationUnit](float a) {
                    return (rotationUnit == RotationUnit::degrees) ? tdm::frad(tdm::fdeg(a)) : tdm::frad(a);
                };
                const auto rad2angle = [rotationUnit](tdm::frad r) {
                    return (rotationUnit == dna::RotationUnit::degrees) ? tdm::fdeg(r).value : r.value;
                };

                const auto jointRotationMatrix = tdm::rotate(angle2rad(jointNeutralRotation.x),
                                                             angle2rad(jointNeutralRotation.y),
                                                             angle2rad(jointNeutralRotation.z),
                                                             rotationSequence,
                                                             rotationSigns);
                const auto jointTranslationMatrix =
                    tdm::translate(fvec3{jointNeutralTranslation.x, jointNeutralTranslation.y, jointNeutralTranslation.z});

                const auto transformMatrix = jointRotationMatrix * jointTranslationMatrix * rotationMatrix;
                const auto t = extractTranslationVector(transformMatrix);
                const auto r = extractRotationVector(transformMatrix);

                output->setNeutralJointRotation(jointIndex, Vector3{rad2angle(r[0]), rad2angle(r[1]), rad2angle(r[2])});
                output->setNeutralJointTranslation(jointIndex, Vector3{t[0], t[1], t[2]});
            }
        }
    }

    void rotateVertexPositions(DNACalibDNAReaderImpl* output) {
        const auto rotationSequence = output->getRotationSequence();
        const auto rotationSigns = output->getRotationSign();
        const auto rotationMatrix = getRotationTransformationMatrix(rotationSequence, rotationSigns);
        for (std::uint16_t meshIndex = 0u; meshIndex < output->getMeshCount(); ++meshIndex) {
            const auto xs = output->getVertexPositionXs(meshIndex);
            const auto ys = output->getVertexPositionYs(meshIndex);
            const auto zs = output->getVertexPositionZs(meshIndex);
            assert((xs.size() == ys.size()) && (ys.size() == zs.size()));
            RawVector3Vector mesh{xs, ys, zs, output->getMemoryResource()};
            for (std::size_t i = 0ul; i < mesh.size(); ++i) {
                const tdm::fvec4 vertex{mesh.xs[i], mesh.ys[i], mesh.zs[i], 1.0f};
                const tdm::fvec4 rotatedVertex = vertex * rotationMatrix;
                mesh.xs[i] = rotatedVertex[0];
                mesh.ys[i] = rotatedVertex[1];
                mesh.zs[i] = rotatedVertex[2];
            }
            output->setVertexPositions(meshIndex, std::move(mesh));
        }
    }

    void rotateBlendShapeTargetDeltas(DNACalibDNAReaderImpl* output) {
        const auto rotationSequence = output->getRotationSequence();
        const auto rotationSigns = output->getRotationSign();
        const auto rotationMatrix = getRotationTransformationMatrix(rotationSequence, rotationSigns);
        for (std::uint16_t meshIndex = 0u; meshIndex < output->getMeshCount(); ++meshIndex) {
            for (std::uint16_t blendShapeTargetIndex = 0u; blendShapeTargetIndex < output->getBlendShapeTargetCount(meshIndex);
                 ++blendShapeTargetIndex) {
                const auto xs = output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex);
                const auto ys = output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex);
                const auto zs = output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex);
                assert((xs.size() == ys.size()) && (ys.size() == zs.size()));
                RawVector3Vector deltas{xs, ys, zs, output->getMemoryResource()};
                for (std::size_t i = 0ul; i < deltas.size(); ++i) {
                    const tdm::fvec4 delta{deltas.xs[i], deltas.ys[i], deltas.zs[i], 1.0f};
                    const tdm::fvec4 rotatedDelta = delta * rotationMatrix;
                    deltas.xs[i] = rotatedDelta[0];
                    deltas.ys[i] = rotatedDelta[1];
                    deltas.zs[i] = rotatedDelta[2];
                }
                output->setBlendShapeTargetDeltas(meshIndex, blendShapeTargetIndex, std::move(deltas));
            }
        }
    }

private:
    Vector3 degrees;
    Vector3 origin;
};

RotateCommand::RotateCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

RotateCommand::RotateCommand(Vector3 degrees, Vector3 origin, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setRotation(degrees);
    pImpl->setOrigin(origin);
}

RotateCommand::~RotateCommand() = default;
RotateCommand::RotateCommand(RotateCommand&&) = default;
RotateCommand& RotateCommand::operator=(RotateCommand&&) = default;

void RotateCommand::setRotation(Vector3 degrees) {
    pImpl->setRotation(degrees);
}

void RotateCommand::setOrigin(Vector3 origin) {
    pImpl->setOrigin(origin);
}

void RotateCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
