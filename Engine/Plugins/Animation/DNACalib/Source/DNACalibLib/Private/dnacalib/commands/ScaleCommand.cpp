// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/ScaleCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class ScaleCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        origin{},
        scale{1.0f} {
    }

    void setScale(float scale_) {
        scale = scale_;
    }

    void setOrigin(Vector3 origin_) {
        origin = origin_;
    }

    void run(DNACalibDNAReaderImpl* output) {
        if (scale != 1.0f) {
            scaleNeutralJoints(output);
            scaleJointBehavior(output);
            scaleGeometry(output);
        }
    }

private:
    void scaleNeutralJoints(DNACalibDNAReaderImpl* output) {
        const auto xs = output->getNeutralJointTranslationXs();
        const auto ys = output->getNeutralJointTranslationYs();
        const auto zs = output->getNeutralJointTranslationZs();
        assert((xs.size() == ys.size()) && (ys.size() == zs.size()));
        RawVector3Vector translations{xs, ys, zs, output->getMemoryResource()};
        for (std::uint16_t i = 0ul; i < translations.size(); ++i) {
            const bool isRootJoint = (output->getJointParentIndex(i) == i);
            if (isRootJoint) {
                translations.xs[i] -= origin.x;
                translations.ys[i] -= origin.y;
                translations.zs[i] -= origin.z;

                translations.xs[i] *= scale;
                translations.ys[i] *= scale;
                translations.zs[i] *= scale;

                translations.xs[i] += origin.x;
                translations.ys[i] += origin.y;
                translations.zs[i] += origin.z;
            } else {
                translations.xs[i] *= scale;
                translations.ys[i] *= scale;
                translations.zs[i] *= scale;
            }
        }
        output->setNeutralJointTranslations(std::move(translations));
    }

    void scaleJointBehavior(DNACalibDNAReaderImpl* output) {
        constexpr std::uint16_t jointAttributeCount = 9u;
        constexpr std::uint16_t rotationOffset = 3u;

        for (std::uint16_t jointGroupIndex = 0u; jointGroupIndex < output->getJointGroupCount(); ++jointGroupIndex) {
            const auto values = output->getJointGroupValues(jointGroupIndex);
            const auto outputIndices = output->getJointGroupOutputIndices(jointGroupIndex);
            const auto inputIndices = output->getJointGroupInputIndices(jointGroupIndex);
            const auto columnCount = inputIndices.size();
            AlignedDynArray<float> newValues{values.begin(), values.end(), output->getMemoryResource()};
            for (std::size_t row = 0ul; row < outputIndices.size(); ++row) {
                // Only the translation attributes need to be scaled
                const auto relAttributeIndex = (outputIndices[row] % jointAttributeCount);
                if (relAttributeIndex < rotationOffset) {
                    for (std::size_t column = 0ul; column < columnCount; ++column) {
                        newValues[row * columnCount + column] *= scale;
                    }
                }
            }
            output->setJointGroupValues(jointGroupIndex, std::move(newValues));
        }
    }

    void scaleGeometry(DNACalibDNAReaderImpl* output) {
        for (std::uint16_t meshIndex = 0u; meshIndex < output->getMeshCount(); ++meshIndex) {
            scaleVertexPositions(output, meshIndex);
            scaleBlendShapeTargetDeltas(output, meshIndex);
        }
    }

    void scaleVertexPositions(DNACalibDNAReaderImpl* output, std::uint16_t meshIndex) {
        const auto xs = output->getVertexPositionXs(meshIndex);
        const auto ys = output->getVertexPositionYs(meshIndex);
        const auto zs = output->getVertexPositionZs(meshIndex);
        assert((xs.size() == ys.size()) && (ys.size() == zs.size()));
        RawVector3Vector vertices{xs, ys, zs, output->getMemoryResource()};
        for (std::uint32_t i = 0u; i < vertices.size(); ++i) {
            vertices.xs[i] -= origin.x;
            vertices.ys[i] -= origin.y;
            vertices.zs[i] -= origin.z;

            vertices.xs[i] *= scale;
            vertices.ys[i] *= scale;
            vertices.zs[i] *= scale;

            vertices.xs[i] += origin.x;
            vertices.ys[i] += origin.y;
            vertices.zs[i] += origin.z;
        }
        output->setVertexPositions(meshIndex, std::move(vertices));
    }

    void scaleBlendShapeTargetDeltas(DNACalibDNAReaderImpl* output, std::uint16_t meshIndex) {
        for (std::uint16_t blendShapeTargetIndex = 0u; blendShapeTargetIndex < output->getBlendShapeTargetCount(meshIndex);
             ++blendShapeTargetIndex) {
            const auto xs = output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex);
            const auto ys = output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex);
            const auto zs = output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex);
            assert((xs.size() == ys.size()) && (ys.size() == zs.size()));
            RawVector3Vector deltas{xs, ys, zs, output->getMemoryResource()};
            for (std::uint32_t i = 0u; i < deltas.size(); ++i) {
                deltas.xs[i] *= scale;
                deltas.ys[i] *= scale;
                deltas.zs[i] *= scale;
            }
            output->setBlendShapeTargetDeltas(meshIndex, blendShapeTargetIndex, std::move(deltas));
        }
    }

private:
    Vector3 origin;
    float scale;
};

ScaleCommand::ScaleCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

ScaleCommand::ScaleCommand(float scale, Vector3 origin, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setScale(scale);
    pImpl->setOrigin(origin);
}

ScaleCommand::~ScaleCommand() = default;
ScaleCommand::ScaleCommand(ScaleCommand&&) = default;
ScaleCommand& ScaleCommand::operator=(ScaleCommand&&) = default;

void ScaleCommand::setScale(float scale) {
    pImpl->setScale(scale);
}

void ScaleCommand::setOrigin(Vector3 origin) {
    pImpl->setOrigin(origin);
}

void ScaleCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
