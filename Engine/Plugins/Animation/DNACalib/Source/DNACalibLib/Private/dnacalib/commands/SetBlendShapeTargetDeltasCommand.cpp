// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/SetBlendShapeTargetDeltasCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/TypeDefs.h"
#include "dnacalib/commands/SupportFactories.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"
#include "dnacalib/utils/FormatString.h"

#include <cstdint>
#include <numeric>

namespace dnac {

class SetBlendShapeTargetDeltasCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        deltas{memRes_},
        vertexIndices{memRes_},
        masks{memRes_},
        operation{VectorOperation::Interpolate},
        meshIndex{},
        blendShapeTargetIndex{} {
    }

    void setMeshIndex(std::uint16_t meshIndex_) {
        meshIndex = meshIndex_;
    }

    void setBlendShapeTargetIndex(std::uint16_t blendShapeTargetIndex_) {
        blendShapeTargetIndex = blendShapeTargetIndex_;
    }

    void setDeltas(ConstArrayView<Vector3> deltas_) {
        deltas.assign(deltas_.begin(), deltas_.end());
    }

    void setDeltas(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs) {
        deltas.xs.assign(xs.begin(), xs.end());
        deltas.ys.assign(ys.begin(), ys.end());
        deltas.zs.assign(zs.begin(), zs.end());
    }

    void setVertexIndices(ConstArrayView<std::uint32_t> vertexIndices_) {
        vertexIndices.assign(vertexIndices_.begin(), vertexIndices_.end());
    }

    void setMasks(ConstArrayView<float> masks_) {
        masks.assign(masks_.begin(), masks_.end());
    }

    void setOperation(VectorOperation operation_) {
        operation = operation_;
    }

    void run(DNACalibDNAReaderImpl* output) {
        status.reset();
        auto getWeight = WeightGetterFactory::create(masks);
        auto op = OperationFactory::create(operation);
        computeBlendShapeTargetDeltas(op, getWeight, output);
    }

private:
    static void densify(RawVector3Vector& bsDeltas, Vector<std::uint32_t>& bsVertexIndices, std::uint32_t vertexCount) {
        const auto deltaCount = static_cast<std::uint32_t>(bsDeltas.size());
        bsDeltas.resize(vertexCount);
        bsVertexIndices.resize(vertexCount);
        for (std::uint32_t j = deltaCount; j > 0; --j) {
            const auto i = j - 1;
            const auto srcDelta = Vector3{bsDeltas.xs[i], bsDeltas.ys[i], bsDeltas.zs[i]};
            bsDeltas.xs[i] = {};
            bsDeltas.ys[i] = {};
            bsDeltas.zs[i] = {};
            bsDeltas.xs[bsVertexIndices[i]] = srcDelta.x;
            bsDeltas.ys[bsVertexIndices[i]] = srcDelta.y;
            bsDeltas.zs[bsVertexIndices[i]] = srcDelta.z;
        }
        std::iota(bsVertexIndices.begin(), bsVertexIndices.end(), 0u);
    }

    static void sparsify(RawVector3Vector& bsDeltas, Vector<std::uint32_t>& bsVertexIndices, float threshold) {
        const float threshold2 = threshold * threshold;
        std::uint32_t di = 0u;
        for (std::uint32_t si = 0u; si < bsVertexIndices.size(); si++) {
            const auto sourceDelta = tdm::fvec3{bsDeltas.xs[si], bsDeltas.ys[si], bsDeltas.zs[si]};
            const float magnitude2 = tdm::dot(sourceDelta, sourceDelta);
            if (magnitude2 > threshold2) {
                bsVertexIndices[di] = bsVertexIndices[si];
                bsDeltas.xs[di] = sourceDelta[0];
                bsDeltas.ys[di] = sourceDelta[1];
                bsDeltas.zs[di] = sourceDelta[2];
                ++di;
            }
        }
        bsDeltas.resize(di);
        bsVertexIndices.resize(di);
    }

    template<typename FOperation, typename FWeightGetter>
    void computeBlendShapeTargetDeltas(FOperation op, FWeightGetter getWeight, DNACalibDNAReaderImpl* output) {
        const auto xs = output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex);
        const auto ys = output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex);
        const auto zs = output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex);
        const auto vtxIndices = output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex);
        assert((xs.size() == ys.size()) && (ys.size() == zs.size()) && (xs.size() == vtxIndices.size()));
        RawVector3Vector bsDeltas{xs, ys, zs, output->getMemoryResource()};
        Vector<std::uint32_t> bsVertexIndices{output->getMemoryResource()};
        bsVertexIndices.assign(vtxIndices.begin(), vtxIndices.end());

        // If no vertex indices were set, try using existing ones. The condition that must be met in that case is that number
        // of set deltas equals the number of existing vertex indices.
        if (vertexIndices.empty()) {
            if (deltas.size() != bsVertexIndices.size()) {
                const auto message = formatString(output->getMemoryResource(),
                                                  "No vertex indices set. Current vertex indices in DNA will not be used, as "
                                                  "their number (%hu) differs from the number of set deltas (%hu).",
                                                  bsVertexIndices.size(),
                                                  deltas.size());
                status.set(NoVertexIndicesSetError, message.c_str());
                return;
            }
            vertexIndices.assign(bsVertexIndices.begin(), bsVertexIndices.end());
        }

        const auto vertexCount = output->getVertexPositionCount(meshIndex);
        for (const auto vi : vertexIndices) {
            if (vi >= vertexCount) {
                const auto message = formatString(output->getMemoryResource(),
                                                  "Vertex index (%hu) is out of bounds. Vertex count is (%hu).",
                                                  vi,
                                                  vertexCount);
                status.set(VertexIndicesOutOfBoundsError, message.c_str());
                return;
            }
        }

        // Densify current blend shapes from DNA
        densify(bsDeltas, bsVertexIndices, vertexCount);

        if (deltas.size() != vertexIndices.size()) {
            const auto message = formatString(output->getMemoryResource(),
                                              "Number of set deltas (%hu) differs from number of set vertex indices (%hu).",
                                              deltas.size(),
                                              vertexIndices.size());
            status.set(DeltasVertexIndicesCountMismatch, message.c_str());
            return;
        }
        if (!masks.empty() && (deltas.size() != masks.size())) {
            const auto message = formatString(output->getMemoryResource(),
                                              "Number of set deltas (%hu) differs from number of set masks (%hu).",
                                              deltas.size(),
                                              masks.size());
            status.set(DeltasMasksCountMismatch, message.c_str());
            return;
        }

        // Compute operation
        assert(bsDeltas.size() == vertexCount);
        for (std::uint32_t i = 0u; i < vertexIndices.size(); ++i) {
            const auto index = vertexIndices[i];
            const float weight = getWeight(masks.data(), i);
            bsDeltas.xs[index] = op(bsDeltas.xs[index], deltas.xs[i], weight);
            bsDeltas.ys[index] = op(bsDeltas.ys[index], deltas.ys[i], weight);
            bsDeltas.zs[index] = op(bsDeltas.zs[index], deltas.zs[i], weight);
        }

        // Sparsify result
        sparsify(bsDeltas, bsVertexIndices, 0.0f);

        // Set new deltas and vertex indices to output DNA
        output->setBlendShapeTargetDeltas(meshIndex, blendShapeTargetIndex, std::move(bsDeltas));
        output->setBlendShapeTargetVertexIndices(meshIndex,
                                                 blendShapeTargetIndex,
                                                 ConstArrayView<std::uint32_t>{bsVertexIndices});
    }

private:
    static sc::StatusProvider status;

    RawVector3Vector deltas;
    Vector<std::uint32_t> vertexIndices;
    Vector<float> masks;
    VectorOperation operation;
    std::uint16_t meshIndex;
    std::uint16_t blendShapeTargetIndex;
};

const sc::StatusCode SetBlendShapeTargetDeltasCommand::VertexIndicesOutOfBoundsError{3101, "%s"};
const sc::StatusCode SetBlendShapeTargetDeltasCommand::NoVertexIndicesSetError{3102, "%s"};
const sc::StatusCode SetBlendShapeTargetDeltasCommand::DeltasVertexIndicesCountMismatch{3103, "%s"};
const sc::StatusCode SetBlendShapeTargetDeltasCommand::DeltasMasksCountMismatch{3104, "%s"};

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wglobal-constructors"
#endif
sc::StatusProvider SetBlendShapeTargetDeltasCommand::Impl::status{VertexIndicesOutOfBoundsError,
                                                                  NoVertexIndicesSetError,
                                                                  DeltasVertexIndicesCountMismatch,
                                                                  DeltasMasksCountMismatch};
#ifdef __clang__
    #pragma clang diagnostic pop
#endif

SetBlendShapeTargetDeltasCommand::SetBlendShapeTargetDeltasCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

SetBlendShapeTargetDeltasCommand::SetBlendShapeTargetDeltasCommand(std::uint16_t meshIndex,
                                                                   std::uint16_t blendShapeTargetIndex,
                                                                   ConstArrayView<Vector3> deltas,
                                                                   ConstArrayView<std::uint32_t> vertexIndices,
                                                                   VectorOperation operation,
                                                                   MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndex(meshIndex);
    pImpl->setBlendShapeTargetIndex(blendShapeTargetIndex);
    pImpl->setDeltas(deltas);
    pImpl->setVertexIndices(vertexIndices);
    pImpl->setOperation(operation);
}

SetBlendShapeTargetDeltasCommand::SetBlendShapeTargetDeltasCommand(std::uint16_t meshIndex,
                                                                   std::uint16_t blendShapeTargetIndex,
                                                                   ConstArrayView<float> xs,
                                                                   ConstArrayView<float> ys,
                                                                   ConstArrayView<float> zs,
                                                                   ConstArrayView<std::uint32_t> vertexIndices,
                                                                   VectorOperation operation,
                                                                   MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndex(meshIndex);
    pImpl->setBlendShapeTargetIndex(blendShapeTargetIndex);
    pImpl->setDeltas(xs, ys, zs);
    pImpl->setVertexIndices(vertexIndices);
    pImpl->setOperation(operation);
}

SetBlendShapeTargetDeltasCommand::SetBlendShapeTargetDeltasCommand(std::uint16_t meshIndex,
                                                                   std::uint16_t blendShapeTargetIndex,
                                                                   ConstArrayView<Vector3> deltas,
                                                                   ConstArrayView<std::uint32_t> vertexIndices,
                                                                   ConstArrayView<float> masks,
                                                                   VectorOperation operation,
                                                                   MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndex(meshIndex);
    pImpl->setBlendShapeTargetIndex(blendShapeTargetIndex);
    pImpl->setDeltas(deltas);
    pImpl->setVertexIndices(vertexIndices);
    pImpl->setMasks(masks);
    pImpl->setOperation(operation);
}

SetBlendShapeTargetDeltasCommand::SetBlendShapeTargetDeltasCommand(std::uint16_t meshIndex,
                                                                   std::uint16_t blendShapeTargetIndex,
                                                                   ConstArrayView<float> xs,
                                                                   ConstArrayView<float> ys,
                                                                   ConstArrayView<float> zs,
                                                                   ConstArrayView<std::uint32_t> vertexIndices,
                                                                   ConstArrayView<float> masks,
                                                                   VectorOperation operation,
                                                                   MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndex(meshIndex);
    pImpl->setBlendShapeTargetIndex(blendShapeTargetIndex);
    pImpl->setDeltas(xs, ys, zs);
    pImpl->setVertexIndices(vertexIndices);
    pImpl->setMasks(masks);
    pImpl->setOperation(operation);
}

SetBlendShapeTargetDeltasCommand::~SetBlendShapeTargetDeltasCommand() = default;
SetBlendShapeTargetDeltasCommand::SetBlendShapeTargetDeltasCommand(SetBlendShapeTargetDeltasCommand&&) = default;
SetBlendShapeTargetDeltasCommand& SetBlendShapeTargetDeltasCommand::operator=(SetBlendShapeTargetDeltasCommand&&) = default;

void SetBlendShapeTargetDeltasCommand::setMeshIndex(std::uint16_t meshIndex) {
    pImpl->setMeshIndex(meshIndex);
}

void SetBlendShapeTargetDeltasCommand::setBlendShapeTargetIndex(std::uint16_t blendShapeTargetIndex) {
    pImpl->setBlendShapeTargetIndex(blendShapeTargetIndex);
}

void SetBlendShapeTargetDeltasCommand::setDeltas(ConstArrayView<Vector3> deltas) {
    pImpl->setDeltas(deltas);
}

void SetBlendShapeTargetDeltasCommand::setDeltas(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs) {
    pImpl->setDeltas(xs, ys, zs);
}

void SetBlendShapeTargetDeltasCommand::setVertexIndices(ConstArrayView<std::uint32_t> vertexIndices) {
    pImpl->setVertexIndices(vertexIndices);
}

void SetBlendShapeTargetDeltasCommand::setMasks(ConstArrayView<float> masks) {
    pImpl->setMasks(masks);
}

void SetBlendShapeTargetDeltasCommand::setOperation(VectorOperation operation) {
    pImpl->setOperation(operation);
}

void SetBlendShapeTargetDeltasCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
