// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/SetLODsCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class SetLODsCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        lods{memRes_} {
    }

    void setLODs(ConstArrayView<std::uint16_t> lods_) {
        lods.assign(lods_.begin(), lods_.end());
    }

    void run(DNACalibDNAReaderImpl* output) {
        auto buffer = makeScoped<MemoryStream>(getMemoryResource());
        auto writer = makeScoped<BinaryStreamWriter>(buffer.get(), getMemoryResource());
        writer->setFrom(output);
        writer->write();
        buffer->seek(0ul);
        dna::Configuration config;
        config.lods = lods;
        auto reader = makeScoped<BinaryStreamReader>(buffer.get(), config, getMemoryResource());
        reader->read();
        output->setFrom(reader.get());
    }

private:
    Vector<std::uint16_t> lods;
};

SetLODsCommand::SetLODsCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

SetLODsCommand::SetLODsCommand(ConstArrayView<std::uint16_t> lods, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setLODs(lods);
}

SetLODsCommand::~SetLODsCommand() = default;
SetLODsCommand::SetLODsCommand(SetLODsCommand&&) = default;
SetLODsCommand& SetLODsCommand::operator=(SetLODsCommand&&) = default;

void SetLODsCommand::setLODs(ConstArrayView<std::uint16_t> lods) {
    pImpl->setLODs(lods);
}

void SetLODsCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
