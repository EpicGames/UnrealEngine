// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/SetNeutralJointRotationsCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class SetNeutralJointRotationsCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        rotations{memRes_} {
    }

    void setRotations(ConstArrayView<Vector3> rotations_) {
        rotations.assign(rotations_.begin(), rotations_.end());
    }

    void setRotations(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs) {
        rotations.xs.assign(xs.begin(), xs.end());
        rotations.ys.assign(ys.begin(), ys.end());
        rotations.zs.assign(zs.begin(), zs.end());
    }

    void run(DNACalibDNAReaderImpl* output) {
        output->setNeutralJointRotations(ConstArrayView<float>{rotations.xs},
                                         ConstArrayView<float>{rotations.ys},
                                         ConstArrayView<float>{rotations.zs});
    }

private:
    RawVector3Vector rotations;
};

SetNeutralJointRotationsCommand::SetNeutralJointRotationsCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

SetNeutralJointRotationsCommand::SetNeutralJointRotationsCommand(ConstArrayView<Vector3> rotations, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setRotations(rotations);
}

SetNeutralJointRotationsCommand::SetNeutralJointRotationsCommand(ConstArrayView<float> xs,
                                                                 ConstArrayView<float> ys,
                                                                 ConstArrayView<float> zs,
                                                                 MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setRotations(xs, ys, zs);
}

SetNeutralJointRotationsCommand::~SetNeutralJointRotationsCommand() = default;
SetNeutralJointRotationsCommand::SetNeutralJointRotationsCommand(SetNeutralJointRotationsCommand&&) = default;
SetNeutralJointRotationsCommand& SetNeutralJointRotationsCommand::operator=(SetNeutralJointRotationsCommand&&) = default;

void SetNeutralJointRotationsCommand::setRotations(ConstArrayView<Vector3> rotations) {
    pImpl->setRotations(rotations);
}

void SetNeutralJointRotationsCommand::setRotations(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs) {
    pImpl->setRotations(xs, ys, zs);
}

void SetNeutralJointRotationsCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
