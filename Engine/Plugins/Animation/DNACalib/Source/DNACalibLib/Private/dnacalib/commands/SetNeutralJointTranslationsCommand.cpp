// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/SetNeutralJointTranslationsCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class SetNeutralJointTranslationsCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        translations{memRes_} {
    }

    void setTranslations(ConstArrayView<Vector3> translations_) {
        translations.assign(translations_.begin(), translations_.end());
    }

    void setTranslations(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs) {
        translations.xs.assign(xs.begin(), xs.end());
        translations.ys.assign(ys.begin(), ys.end());
        translations.zs.assign(zs.begin(), zs.end());
    }

    void run(DNACalibDNAReaderImpl* output) {
        output->setNeutralJointTranslations(ConstArrayView<float>{translations.xs},
                                            ConstArrayView<float>{translations.ys},
                                            ConstArrayView<float>{translations.zs});
    }

private:
    RawVector3Vector translations;
};

SetNeutralJointTranslationsCommand::SetNeutralJointTranslationsCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

SetNeutralJointTranslationsCommand::SetNeutralJointTranslationsCommand(ConstArrayView<Vector3> translations,
                                                                       MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setTranslations(translations);
}

SetNeutralJointTranslationsCommand::SetNeutralJointTranslationsCommand(ConstArrayView<float> xs,
                                                                       ConstArrayView<float> ys,
                                                                       ConstArrayView<float> zs,
                                                                       MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setTranslations(xs, ys, zs);
}

SetNeutralJointTranslationsCommand::~SetNeutralJointTranslationsCommand() = default;
SetNeutralJointTranslationsCommand::SetNeutralJointTranslationsCommand(SetNeutralJointTranslationsCommand&&) = default;
SetNeutralJointTranslationsCommand& SetNeutralJointTranslationsCommand::operator=(SetNeutralJointTranslationsCommand&&) = default;

void SetNeutralJointTranslationsCommand::setTranslations(ConstArrayView<Vector3> translations) {
    pImpl->setTranslations(translations);
}

void SetNeutralJointTranslationsCommand::setTranslations(ConstArrayView<float> xs,
                                                         ConstArrayView<float> ys,
                                                         ConstArrayView<float> zs) {
    pImpl->setTranslations(xs, ys, zs);
}

void SetNeutralJointTranslationsCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
