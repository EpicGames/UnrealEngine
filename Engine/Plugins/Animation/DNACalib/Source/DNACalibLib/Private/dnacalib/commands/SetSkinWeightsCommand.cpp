// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/SetSkinWeightsCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class SetSkinWeightsCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        weights{memRes_},
        jointIndices{memRes_},
        meshIndex{},
        vertexIndex{} {
    }

    void setMeshIndex(std::uint16_t meshIndex_) {
        meshIndex = meshIndex_;
    }

    void setVertexIndex(std::uint32_t vertexIndex_) {
        vertexIndex = vertexIndex_;
    }

    void setWeights(ConstArrayView<float> weights_) {
        weights.assign(weights_.begin(), weights_.end());
    }

    void setJointIndices(ConstArrayView<std::uint16_t> jointIndices_) {
        jointIndices.assign(jointIndices_.begin(), jointIndices_.end());
    }

    void run(DNACalibDNAReaderImpl* output) {
        output->setSkinWeightsValues(meshIndex, vertexIndex, weights.data(), static_cast<std::uint16_t>(weights.size()));
        output->setSkinWeightsJointIndices(meshIndex,
                                           vertexIndex,
                                           jointIndices.data(),
                                           static_cast<std::uint16_t>(jointIndices.size()));
    }

private:
    Vector<float> weights;
    Vector<std::uint16_t> jointIndices;
    std::uint16_t meshIndex;
    std::uint32_t vertexIndex;
};

SetSkinWeightsCommand::SetSkinWeightsCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

SetSkinWeightsCommand::SetSkinWeightsCommand(std::uint16_t meshIndex,
                                             std::uint32_t vertexIndex,
                                             ConstArrayView<float> weights,
                                             ConstArrayView<std::uint16_t> jointIndices,
                                             MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndex(meshIndex);
    pImpl->setVertexIndex(vertexIndex);
    pImpl->setWeights(weights);
    pImpl->setJointIndices(jointIndices);
}

SetSkinWeightsCommand::~SetSkinWeightsCommand() = default;
SetSkinWeightsCommand::SetSkinWeightsCommand(SetSkinWeightsCommand&&) = default;
SetSkinWeightsCommand& SetSkinWeightsCommand::operator=(SetSkinWeightsCommand&&) = default;

void SetSkinWeightsCommand::setMeshIndex(std::uint16_t meshIndex) {
    pImpl->setMeshIndex(meshIndex);
}

void SetSkinWeightsCommand::setVertexIndex(std::uint32_t vertexIndex) {
    pImpl->setVertexIndex(vertexIndex);
}

void SetSkinWeightsCommand::setWeights(ConstArrayView<float> weights) {
    pImpl->setWeights(weights);
}

void SetSkinWeightsCommand::setJointIndices(ConstArrayView<std::uint16_t> jointIndices) {
    pImpl->setJointIndices(jointIndices);
}

void SetSkinWeightsCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
