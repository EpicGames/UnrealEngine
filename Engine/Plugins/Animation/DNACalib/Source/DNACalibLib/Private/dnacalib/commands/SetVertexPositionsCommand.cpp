// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/SetVertexPositionsCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/TypeDefs.h"
#include "dnacalib/commands/SupportFactories.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"
#include "dnacalib/utils/FormatString.h"

namespace dnac {

class SetVertexPositionsCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        positions{memRes_},
        masks{memRes_},
        operation{VectorOperation::Interpolate},
        meshIndex{} {
    }

    void setMeshIndex(std::uint16_t meshIndex_) {
        meshIndex = meshIndex_;
    }

    void setPositions(ConstArrayView<Vector3> positions_) {
        positions.assign(positions_.begin(), positions_.end());
    }

    void setPositions(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs) {
        positions.xs.assign(xs.begin(), xs.end());
        positions.ys.assign(ys.begin(), ys.end());
        positions.zs.assign(zs.begin(), zs.end());
    }

    void setMasks(ConstArrayView<float> masks_) {
        masks.assign(masks_.begin(), masks_.end());
    }

    void setOperation(VectorOperation operation_) {
        operation = operation_;
    }

    void run(DNACalibDNAReaderImpl* output) {
        status.reset();
        auto getWeight = WeightGetterFactory::create(masks);
        auto op = OperationFactory::create(operation);
        computeVertexPositions(op, getWeight, output);
    }

private:
    template<typename FOperation, typename FWeightGetter>
    void computeVertexPositions(FOperation op, FWeightGetter getWeight, DNACalibDNAReaderImpl* output) {
        const auto xs = output->getVertexPositionXs(meshIndex);
        const auto ys = output->getVertexPositionYs(meshIndex);
        const auto zs = output->getVertexPositionZs(meshIndex);
        assert((xs.size() == ys.size()) && (ys.size() == zs.size()));
        RawVector3Vector result{xs, ys, zs, output->getMemoryResource()};
        // This accounts for the case when output is empty
        result.resize(positions.size(), 0.0f);
        if (!masks.empty() && (positions.size() != masks.size())) {
            const auto message = formatString(output->getMemoryResource(),
                                              "Number of set positions (%hu) differs from number of set masks (%hu).",
                                              positions.size(),
                                              masks.size());
            status.set(PositionsMasksCountMismatch, message.c_str());
            return;
        }
        for (std::uint32_t i = 0u; i < positions.size(); ++i) {
            const float weight = getWeight(masks.data(), i);
            result.xs[i] = op(result.xs[i], positions.xs[i], weight);
            result.ys[i] = op(result.ys[i], positions.ys[i], weight);
            result.zs[i] = op(result.zs[i], positions.zs[i], weight);
        }
        output->setVertexPositions(meshIndex, std::move(result));
    }

private:
    static sc::StatusProvider status;

    RawVector3Vector positions;
    Vector<float> masks;
    VectorOperation operation;
    std::uint16_t meshIndex;
};

const sc::StatusCode SetVertexPositionsCommand::PositionsMasksCountMismatch{3201, "%s"};

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wglobal-constructors"
#endif
sc::StatusProvider SetVertexPositionsCommand::Impl::status{PositionsMasksCountMismatch};
#ifdef __clang__
    #pragma clang diagnostic pop
#endif

SetVertexPositionsCommand::SetVertexPositionsCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

SetVertexPositionsCommand::SetVertexPositionsCommand(std::uint16_t meshIndex,
                                                     ConstArrayView<Vector3> positions,
                                                     VectorOperation operation,
                                                     MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndex(meshIndex);
    pImpl->setPositions(positions);
    pImpl->setOperation(operation);
}

SetVertexPositionsCommand::SetVertexPositionsCommand(std::uint16_t meshIndex,
                                                     ConstArrayView<float> xs,
                                                     ConstArrayView<float> ys,
                                                     ConstArrayView<float> zs,
                                                     VectorOperation operation,
                                                     MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndex(meshIndex);
    pImpl->setPositions(xs, ys, zs);
    pImpl->setOperation(operation);
}

SetVertexPositionsCommand::SetVertexPositionsCommand(std::uint16_t meshIndex,
                                                     ConstArrayView<Vector3> positions,
                                                     ConstArrayView<float> masks,
                                                     VectorOperation operation,
                                                     MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndex(meshIndex);
    pImpl->setPositions(positions);
    pImpl->setMasks(masks);
    pImpl->setOperation(operation);
}

SetVertexPositionsCommand::SetVertexPositionsCommand(std::uint16_t meshIndex,
                                                     ConstArrayView<float> xs,
                                                     ConstArrayView<float> ys,
                                                     ConstArrayView<float> zs,
                                                     ConstArrayView<float> masks,
                                                     VectorOperation operation,
                                                     MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setMeshIndex(meshIndex);
    pImpl->setPositions(xs, ys, zs);
    pImpl->setMasks(masks);
    pImpl->setOperation(operation);
}

SetVertexPositionsCommand::~SetVertexPositionsCommand() = default;
SetVertexPositionsCommand::SetVertexPositionsCommand(SetVertexPositionsCommand&&) = default;
SetVertexPositionsCommand& SetVertexPositionsCommand::operator=(SetVertexPositionsCommand&&) = default;

void SetVertexPositionsCommand::setMeshIndex(std::uint16_t meshIndex) {
    pImpl->setMeshIndex(meshIndex);
}

void SetVertexPositionsCommand::setPositions(ConstArrayView<Vector3> positions) {
    pImpl->setPositions(positions);
}

void SetVertexPositionsCommand::setPositions(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs) {
    pImpl->setPositions(xs, ys, zs);
}

void SetVertexPositionsCommand::setMasks(ConstArrayView<float> masks) {
    pImpl->setMasks(masks);
}

void SetVertexPositionsCommand::setOperation(VectorOperation operation) {
    pImpl->setOperation(operation);
}

void SetVertexPositionsCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
