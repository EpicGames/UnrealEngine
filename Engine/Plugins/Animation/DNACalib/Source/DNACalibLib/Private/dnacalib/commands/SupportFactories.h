// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"
#include "dnacalib/commands/VectorOperations.h"
#include "dnacalib/utils/Extd.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstddef>
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace dnac {

struct WeightGetterFactory {
    using GetterFunc = std::function<float(float*, std::size_t)>;

    static GetterFunc create(const Vector<float>& masks) {
        if (masks.empty()) {
            return [](float* /*unused*/, std::size_t /*unused*/) { return 1.0f; };
        }
        return [](float* pMasks, std::size_t index) { return pMasks[index]; };
    }
};

struct OperationFactory {
    using OpFunc = std::function<float(float, float, float)>;

    static OpFunc create(VectorOperation operation) {
        switch (operation) {
        case VectorOperation::Interpolate: {
            return [](float a, float b, float weight) { return extd::interpolate(a, b, weight); };
        }
        case VectorOperation::Add: {
            return [](float a, float b, float weight) { return a + (b * weight); };
        }
        case VectorOperation::Subtract: {
            return [](float a, float b, float weight) { return a - (b * weight); };
        }
        case VectorOperation::Multiply: {
            return [](float a, float b, float weight) { return a * (b * weight); };
        }
        default: {
            // Unreachable, but needed to silence gcc warnings
            assert(false);
            return {};
        }
        }
    }
};

}  // namespace dnac
