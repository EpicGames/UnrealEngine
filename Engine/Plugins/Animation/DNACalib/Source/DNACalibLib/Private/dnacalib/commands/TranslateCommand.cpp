// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/TranslateCommand.h"

#include "dnacalib/CommandImplBase.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReaderImpl.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class TranslateCommand::Impl : public CommandImplBase<Impl> {
private:
    using Super = CommandImplBase<Impl>;

public:
    explicit Impl(MemoryResource* memRes_) :
        Super{memRes_},
        translation{} {
    }

    void setTranslation(Vector3 translation_) {
        translation = translation_;
    }

    void run(DNACalibDNAReaderImpl* output) {
        if (translation != Vector3{}) {
            translateNeutralJoints(output);
            translateVertexPositions(output);
        }
    }

private:
    void translateNeutralJoints(DNACalibDNAReaderImpl* output) {
        for (std::uint16_t jointIndex = 0u; jointIndex < output->getJointCount(); ++jointIndex) {
            const auto parentIndex = output->getJointParentIndex(jointIndex);
            // Only root joints are translated
            if (jointIndex == parentIndex) {
                const auto joint = output->getNeutralJointTranslation(jointIndex);
                output->setNeutralJointTranslation(jointIndex, joint + translation);
            }
        }
    }

    void translateVertexPositions(DNACalibDNAReaderImpl* output) {
        for (std::uint16_t meshIndex = 0u; meshIndex < output->getMeshCount(); ++meshIndex) {
            const auto xs = output->getVertexPositionXs(meshIndex);
            const auto ys = output->getVertexPositionYs(meshIndex);
            const auto zs = output->getVertexPositionZs(meshIndex);
            assert((xs.size() == ys.size()) && (ys.size() == zs.size()));
            RawVector3Vector mesh{xs, ys, zs, output->getMemoryResource()};
            for (std::size_t i = 0ul; i < mesh.size(); ++i) {
                mesh.xs[i] += translation.x;
                mesh.ys[i] += translation.y;
                mesh.zs[i] += translation.z;
            }
            output->setVertexPositions(meshIndex, std::move(mesh));
        }
    }

private:
    Vector3 translation;
};

TranslateCommand::TranslateCommand(MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {
}

TranslateCommand::TranslateCommand(Vector3 translation, MemoryResource* memRes) :
    pImpl{makeScoped<Impl>(memRes)} {

    pImpl->setTranslation(translation);
}

TranslateCommand::~TranslateCommand() = default;
TranslateCommand::TranslateCommand(TranslateCommand&&) = default;
TranslateCommand& TranslateCommand::operator=(TranslateCommand&&) = default;

void TranslateCommand::setTranslation(Vector3 translation) {
    pImpl->setTranslation(translation);
}

void TranslateCommand::run(DNACalibDNAReader* output) {
    pImpl->run(static_cast<DNACalibDNAReaderImpl*>(output));
}

}  // namespace dnac
