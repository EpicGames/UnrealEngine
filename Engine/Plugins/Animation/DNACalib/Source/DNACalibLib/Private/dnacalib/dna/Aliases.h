// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/dna/DNA.h"

namespace dnac {

using dna::ActivationFunction;
using dna::Archetype;
using dna::AutomaticRadius;
using dna::CoordinateSystem;
using dna::Delta;
using dna::Direction;
using dna::Gender;
using dna::MachineLearnedBehaviorOperationType;
using dna::MachineLearnedBehaviorParameterKey;
using dna::MeshBlendShapeChannelMapping;
using dna::Normal;
using dna::Position;
using dna::RBFDistanceMethod;
using dna::RBFFunctionType;
using dna::RBFNormalizeMethod;
using dna::RBFSolverType;
using dna::FaceWindingOrder;
using dna::RotationDirection;
using dna::RotationRepresentation;
using dna::RotationSequence;
using dna::RotationSign;
using dna::RotationUnit;
using dna::ScaleRepresentation;
using dna::TextureCoordinate;
using dna::TranslationRepresentation;
using dna::TranslationUnit;
using dna::TwistAxis;
using dna::VertexLayout;

}  // namespace dnac
