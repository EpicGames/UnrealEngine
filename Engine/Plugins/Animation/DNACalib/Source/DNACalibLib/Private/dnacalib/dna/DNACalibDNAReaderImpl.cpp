// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/dna/DNACalibDNAReaderImpl.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/filters/AnimatedMapFilter.h"
#include "dnacalib/dna/filters/BlendShapeFilter.h"
#include "dnacalib/dna/filters/JointFilter.h"
#include "dnacalib/dna/filters/MeshFilter.h"
#include "dnacalib/utils/Extd.h"

namespace dnac {

namespace {

using namespace dna;

}  // namespace

DNACalibDNAReader::~DNACalibDNAReader() = default;

DNACalibDNAReaderImpl::~DNACalibDNAReaderImpl() = default;

DNACalibDNAReader* DNACalibDNAReader::create(MemoryResource* memRes) {
    PolyAllocator<DNACalibDNAReaderImpl> alloc{memRes};
    return alloc.newObject(memRes);
}

DNACalibDNAReader* DNACalibDNAReader::create(const dna::Reader* reader, MemoryResource* memRes) {
    auto instance = static_cast<DNACalibDNAReaderImpl*>(create(memRes));
    instance->setFrom(reader, dna::DataLayer::All, dna::UnknownLayerPolicy::Preserve, memRes);
    return instance;
}

void DNACalibDNAReader::destroy(DNACalibDNAReader* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto ptr = static_cast<DNACalibDNAReaderImpl*>(instance);
    PolyAllocator<DNACalibDNAReaderImpl> alloc{ptr->getMemoryResource()};
    alloc.deleteObject(ptr);
}

DNACalibDNAReaderImpl::DNACalibDNAReaderImpl(MemoryResource* memRes_) :
    BaseImpl{memRes_},
    ReaderImpl{memRes_},
    WriterImpl{memRes_} {
}

void DNACalibDNAReaderImpl::setNeutralJointTranslations(ConstArrayView<float> xs,
                                                        ConstArrayView<float> ys,
                                                        ConstArrayView<float> zs) {
    dna.definition.neutralJointTranslations.xs.assign(xs.begin(), xs.end());
    dna.definition.neutralJointTranslations.ys.assign(ys.begin(), ys.end());
    dna.definition.neutralJointTranslations.zs.assign(zs.begin(), zs.end());
}

void DNACalibDNAReaderImpl::setNeutralJointTranslations(RawVector3Vector&& translations) {
    dna.definition.neutralJointTranslations = std::move(translations);
}

void DNACalibDNAReaderImpl::setNeutralJointTranslation(std::uint16_t index, const Vector3& translation) {
    if (index >= dna.definition.neutralJointTranslations.size()) {
        dna.definition.neutralJointTranslations.xs.resize(index + 1ul, 0.0f);
        dna.definition.neutralJointTranslations.ys.resize(index + 1ul, 0.0f);
        dna.definition.neutralJointTranslations.zs.resize(index + 1ul, 0.0f);
    }
    dna.definition.neutralJointTranslations.xs[index] = translation.x;
    dna.definition.neutralJointTranslations.ys[index] = translation.y;
    dna.definition.neutralJointTranslations.zs[index] = translation.z;
}

void DNACalibDNAReaderImpl::setNeutralJointRotations(ConstArrayView<float> xs,
                                                     ConstArrayView<float> ys,
                                                     ConstArrayView<float> zs) {
    dna.definition.neutralJointRotations.xs.assign(xs.begin(), xs.end());
    dna.definition.neutralJointRotations.ys.assign(ys.begin(), ys.end());
    dna.definition.neutralJointRotations.zs.assign(zs.begin(), zs.end());
}

void DNACalibDNAReaderImpl::setNeutralJointRotations(RawVector3Vector&& rotations) {
    dna.definition.neutralJointRotations = std::move(rotations);
}

void DNACalibDNAReaderImpl::setNeutralJointRotation(std::uint16_t index, const Vector3& rotation) {
    if (index >= dna.definition.neutralJointTranslations.size()) {
        dna.definition.neutralJointRotations.xs.resize(index + 1ul, 0.0f);
        dna.definition.neutralJointRotations.ys.resize(index + 1ul, 0.0f);
        dna.definition.neutralJointRotations.zs.resize(index + 1ul, 0.0f);
    }
    dna.definition.neutralJointRotations.xs[index] = rotation.x;
    dna.definition.neutralJointRotations.ys[index] = rotation.y;
    dna.definition.neutralJointRotations.zs[index] = rotation.z;
}

void DNACalibDNAReaderImpl::setJointGroupValues(std::uint16_t jointGroupIndex, AlignedDynArray<float>&& values) {
    ensureHasSize(dna.behavior.joints.jointGroups, jointGroupIndex + 1ul);
    dna.behavior.joints.jointGroups[jointGroupIndex].values = std::move(values);
}

void DNACalibDNAReaderImpl::setVertexPositions(std::uint16_t meshIndex,
                                               ConstArrayView<float> xs,
                                               ConstArrayView<float> ys,
                                               ConstArrayView<float> zs) {
    ensureHasSize(dna.geometry.meshes, meshIndex + 1ul);
    dna.geometry.meshes[meshIndex].positions.xs.assign(xs.begin(), xs.end());
    dna.geometry.meshes[meshIndex].positions.ys.assign(ys.begin(), ys.end());
    dna.geometry.meshes[meshIndex].positions.zs.assign(zs.begin(), zs.end());
}

void DNACalibDNAReaderImpl::setVertexPositions(std::uint16_t meshIndex, RawVector3Vector&& positions) {
    ensureHasSize(dna.geometry.meshes, meshIndex + 1ul);
    dna.geometry.meshes[meshIndex].positions = std::move(positions);
}

void DNACalibDNAReaderImpl::setBlendShapeTargetDeltas(std::uint16_t meshIndex,
                                                      std::uint16_t blendShapeTargetIndex,
                                                      ConstArrayView<float> xs,
                                                      ConstArrayView<float> ys,
                                                      ConstArrayView<float> zs) {
    ensureHasSize(dna.geometry.meshes, meshIndex + 1ul);
    ensureHasSize(dna.geometry.meshes[meshIndex].blendShapeTargets, blendShapeTargetIndex + 1ul);
    dna.geometry.meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].deltas.xs.assign(xs.begin(), xs.end());
    dna.geometry.meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].deltas.ys.assign(ys.begin(), ys.end());
    dna.geometry.meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].deltas.zs.assign(zs.begin(), zs.end());
}

void DNACalibDNAReaderImpl::setBlendShapeTargetDeltas(std::uint16_t meshIndex,
                                                      std::uint16_t blendShapeTargetIndex,
                                                      RawVector3Vector&& deltas) {
    ensureHasSize(dna.geometry.meshes, meshIndex + 1ul);
    ensureHasSize(dna.geometry.meshes[meshIndex].blendShapeTargets, blendShapeTargetIndex + 1ul);
    dna.geometry.meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].deltas = std::move(deltas);
}

void DNACalibDNAReaderImpl::setBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                             std::uint16_t blendShapeTargetIndex,
                                                             ConstArrayView<std::uint32_t> vertexIndices) {
    ensureHasSize(dna.geometry.meshes, meshIndex + 1ul);
    ensureHasSize(dna.geometry.meshes[meshIndex].blendShapeTargets, blendShapeTargetIndex + 1ul);
    dna.geometry.meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].vertexIndices.assign(vertexIndices.begin(),
                                                                                                 vertexIndices.end());
}

void DNACalibDNAReaderImpl::pruneBlendShapeTargets(float threshold) {
    const float threshold2 = threshold * threshold;
    for (auto& mesh : dna.geometry.meshes) {
        for (auto& bst : mesh.blendShapeTargets) {
            std::size_t di{};
            for (std::size_t si{}; si < bst.deltas.size(); ++si) {
                const float magnitude2 = (bst.deltas.xs[si] * bst.deltas.xs[si]) + (bst.deltas.ys[si] * bst.deltas.ys[si]) +
                                         (bst.deltas.zs[si] * bst.deltas.zs[si]);
                if (magnitude2 > threshold2) {
                    bst.deltas.xs[di] = bst.deltas.xs[si];
                    bst.deltas.ys[di] = bst.deltas.ys[si];
                    bst.deltas.zs[di] = bst.deltas.zs[si];
                    bst.vertexIndices[di] = bst.vertexIndices[si];
                    ++di;
                }
            }
            bst.deltas.resize(di);
            bst.vertexIndices.resize(di);
        }
    }
}

void DNACalibDNAReaderImpl::removeMeshes(ConstArrayView<std::uint16_t> meshIndices) {
    // Filter and remap mesh names and indices
    dna.definition.lodMeshMapping.filterIndices(
        [meshIndices](std::uint16_t value) { return (!extd::contains(meshIndices, value)); });

    // Collect all distinct element position indices that are referenced by the present LODs
    UnorderedSet<std::uint16_t> allowedMeshIndices = dna.definition.lodMeshMapping.getCombinedDistinctIndices(memRes);

    MeshFilter meshFilter{memRes};
    meshFilter.configure(static_cast<std::uint16_t>(dna.definition.meshNames.size()), std::move(allowedMeshIndices));
    meshFilter.apply(dna.definition);
    // Remove mesh geometry
    extd::filter(dna.geometry.meshes, [&meshFilter](const RawMesh& /*unused*/, std::size_t index) {
        return meshFilter.passes(static_cast<std::uint16_t>(index));
    });
    // Repopulate cache of (mesh, blend shape) mapping per LOD
    cache.meshBlendShapeMappingIndices.reset();
    cache.populateMeshBlendShapeMappingIndices(this);
}

void DNACalibDNAReaderImpl::removeJoints(ConstArrayView<std::uint16_t> jointIndices) {
    // To find joints that are not in any LOD, find the joints that are not in LOD 0 (the current max LOD, at index 0), as it
    // contains joints from all lower LODs.
    Vector<std::uint16_t> jointsNotInLOD0{memRes};
    const auto jointIndicesForLOD0 = dna.definition.lodJointMapping.getIndices(0);
    for (std::size_t idx = 0; idx < dna.definition.jointNames.size(); ++idx) {
        if (extd::contains(jointIndices, idx)) {
            // Do not add the joint to remove.
            continue;
        }
        if (std::find(jointIndicesForLOD0.begin(), jointIndicesForLOD0.end(), idx) == jointIndicesForLOD0.end()) {
            jointsNotInLOD0.push_back(static_cast<std::uint16_t>(idx));
        }
    }
    // Filter and remap joint names and indices
    dna.definition.lodJointMapping.filterIndices(
        [jointIndices](std::uint16_t value) { return (!extd::contains(jointIndices, value)); });
    // Collect all distinct element position indices that are referenced by the present LODs
    UnorderedSet<std::uint16_t> allowedJointIndices = dna.definition.lodJointMapping.getCombinedDistinctIndices(memRes);

    // In order to keep joints that are not in any LOD, add them to the list of joints to keep when filtering.
    allowedJointIndices.insert(jointsNotInLOD0.begin(), jointsNotInLOD0.end());

    JointFilter jointFilter{memRes};
    jointFilter.configure(static_cast<std::uint16_t>(dna.definition.jointNames.size()), std::move(allowedJointIndices));
    jointFilter.apply(dna.definition);
    // Filter and remap related joint behavior data
    jointFilter.apply(dna.behavior);
    // Remove skin weights related to this joint and normalize them
    for (auto& mesh : dna.geometry.meshes) {
        for (auto& skinWeights : mesh.skinWeights) {
            jointFilter.apply(skinWeights);
        }
    }

    dna.behavior.joints.rowCount = static_cast<std::uint16_t>(jointFilter.filteredJointCount() * 9u);
}

void DNACalibDNAReaderImpl::removeJointAnimations(ConstArrayView<std::uint16_t> jointIndices) {
    UnorderedSet<std::uint16_t> allowedJointIndices = dna.definition.lodJointMapping.getCombinedDistinctIndices(memRes);
    for (const auto jointIndex : jointIndices) {
        allowedJointIndices.erase(jointIndex);
    }

    JointFilter jointFilter{memRes};
    jointFilter.configure(static_cast<std::uint16_t>(dna.definition.jointNames.size()),
                          std::move(allowedJointIndices),
                          JointFilter::Option::AnimationOnly);
    jointFilter.apply(dna.behavior);
}

void DNACalibDNAReaderImpl::removeBlendShapes(ConstArrayView<std::uint16_t> blendShapeIndices) {
    // Filter blend shapes from LOD blend shape mapping
    dna.definition.lodBlendShapeMapping.filterIndices(
        [blendShapeIndices](std::uint16_t value) { return (!extd::contains(blendShapeIndices, value)); });

    Vector<std::uint16_t> blendShapeLODs{dna.definition.lodBlendShapeMapping.getLODCount(), 0u, memRes};
    for (std::uint16_t lodIndex = 0; lodIndex < static_cast<std::uint16_t>(blendShapeLODs.size()); ++lodIndex) {
        blendShapeLODs[lodIndex] = static_cast<std::uint16_t>(dna.definition.lodBlendShapeMapping.getIndices(lodIndex).size());
    }

    UnorderedSet<std::uint16_t> allowedBlendShapeIndices = dna.definition.lodBlendShapeMapping.getCombinedDistinctIndices(memRes);
    BlendShapeFilter blendShapeFilter{memRes};
    blendShapeFilter.configure(static_cast<std::uint16_t>(dna.definition.blendShapeChannelNames.size()),
                               std::move(allowedBlendShapeIndices),
                               std::move(blendShapeLODs));

    // Remove blend shape from definition
    blendShapeFilter.apply(dna.definition);

    // Remove blend shape from behavior
    blendShapeFilter.apply(dna.behavior);

    // Remove blend shape from geometry
    for (auto& mesh : dna.geometry.meshes) {
        blendShapeFilter.apply(mesh);
    }
}

void DNACalibDNAReaderImpl::removeAnimatedMaps(ConstArrayView<std::uint16_t> animatedMapIndices) {
    // Keep track of animated map indices per LOD before filtering and remapping
    Matrix<std::uint16_t> lodIndices{dna.definition.lodAnimatedMapMapping.getLODCount(), Vector<std::uint16_t>{memRes}, memRes};
    for (std::size_t lodIndex = 0; lodIndex < lodIndices.size(); ++lodIndex) {
        const auto& indices = dna.definition.lodAnimatedMapMapping.getIndices(static_cast<std::uint16_t>(lodIndex));
        lodIndices[lodIndex].assign(indices.begin(), indices.end());
    }
    // Filter and remap animated map names and indices
    dna.definition.lodAnimatedMapMapping.filterIndices(
        [animatedMapIndices](std::uint16_t value) { return (!extd::contains(animatedMapIndices, value)); });

    // Collect all distinct element position indices that are referenced by the present LODs
    UnorderedSet<std::uint16_t> allowedAnimatedMapIndices =
        dna.definition.lodAnimatedMapMapping.getCombinedDistinctIndices(memRes);

    AnimatedMapFilter animatedMapFilter{memRes};
    animatedMapFilter.configure(static_cast<std::uint16_t>(dna.definition.animatedMapNames.size()),
                                std::move(allowedAnimatedMapIndices),
                                std::move(lodIndices));
    animatedMapFilter.apply(dna.definition);
    animatedMapFilter.apply(dna.behavior);
}

void DNACalibDNAReaderImpl::convertToTranslationUnit(TranslationUnit translationUnit) {
    const auto currentTranslationUnit = static_cast<TranslationUnit>(dna.descriptor.translationUnit);
    if (currentTranslationUnit == translationUnit) {
        return;
    }

    constexpr std::size_t unitCount = 2ul;
    // Unit conversion matrix, indexed by [from][to] units
    static const float multipliers[unitCount][unitCount] = {{
                                                                // from cm
                                                                1.0f,  // to cm
                                                                0.01f  // to m
                                                            },
                                                            {
                                                                // from m
                                                                100.0f,  // to cm
                                                                1.0f,    // to m
                                                            }};

    const auto fromIndex = static_cast<std::size_t>(currentTranslationUnit);
    const auto toIndex = static_cast<std::size_t>(translationUnit);
    const auto multiplier = multipliers[fromIndex][toIndex];

    // Convert neutral joint translations
    for (std::size_t i = {}; i < dna.definition.neutralJointTranslations.size(); ++i) {
        dna.definition.neutralJointTranslations.xs[i] *= multiplier;
        dna.definition.neutralJointTranslations.ys[i] *= multiplier;
        dna.definition.neutralJointTranslations.zs[i] *= multiplier;
    }

    // Convert joint group translations
    for (auto& jointGroup : dna.behavior.joints.jointGroups) {
        const auto colCount = jointGroup.inputIndices.size();
        const auto rowCount = jointGroup.outputIndices.size();
        for (std::size_t row = {}; row < rowCount; ++row) {
            // Affect only the first 3 attributes of each joint (tx, ty, tz)
            if ((jointGroup.outputIndices[row] % 9) < 3) {
                for (std::size_t col = {}; col < colCount; ++col) {
                    jointGroup.values[row * colCount + col] *= multiplier;
                }
            }
        }
    }

    for (auto& mesh : dna.geometry.meshes) {
        // Convert vertex positions
        for (std::size_t i = {}; i < mesh.positions.size(); ++i) {
            mesh.positions.xs[i] *= multiplier;
            mesh.positions.ys[i] *= multiplier;
            mesh.positions.zs[i] *= multiplier;
        }

        // Convert blend shape targets
        for (auto& bst : mesh.blendShapeTargets) {
            for (std::size_t i = {}; i < bst.deltas.size(); ++i) {
                bst.deltas.xs[i] *= multiplier;
                bst.deltas.ys[i] *= multiplier;
                bst.deltas.zs[i] *= multiplier;
            }
        }
    }

    dna.descriptor.translationUnit = static_cast<std::uint16_t>(translationUnit);
}

void DNACalibDNAReaderImpl::convertToRotationUnit(RotationUnit rotationUnit) {
    const auto currentRotationUnit = static_cast<RotationUnit>(dna.descriptor.rotationUnit);
    if (currentRotationUnit == rotationUnit) {
        return;
    }

    constexpr std::size_t unitCount = 2ul;
    // Unit conversion matrix, indexed by [from][to] units
    static const float multipliers[unitCount][unitCount] = {{
                                                                // from degrees
                                                                1.0f,       // to degrees
                                                                0.0174533f  // to radians
                                                            },
                                                            {
                                                                // from radians
                                                                57.2958f,  // to degrees
                                                                1.0f,      // to radians
                                                            }};

    const auto fromIndex = static_cast<std::size_t>(currentRotationUnit);
    const auto toIndex = static_cast<std::size_t>(rotationUnit);
    const auto multiplier = multipliers[fromIndex][toIndex];

    // Convert neutral joint rotations
    for (std::size_t i = {}; i < dna.definition.neutralJointRotations.size(); ++i) {
        dna.definition.neutralJointRotations.xs[i] *= multiplier;
        dna.definition.neutralJointRotations.ys[i] *= multiplier;
        dna.definition.neutralJointRotations.zs[i] *= multiplier;
    }

    // Convert joint group rotations
    for (auto& jointGroup : dna.behavior.joints.jointGroups) {
        const auto colCount = jointGroup.inputIndices.size();
        const auto rowCount = jointGroup.outputIndices.size();
        for (std::size_t row = {}; row < rowCount; ++row) {
            // Affect only the middle 3 attributes of each joint (rx, ry, rz)
            if (((jointGroup.outputIndices[row] % 9) > 2) && ((jointGroup.outputIndices[row] % 9) < 6)) {
                for (std::size_t col = {}; col < colCount; ++col) {
                    jointGroup.values[row * colCount + col] *= multiplier;
                }
            }
        }
    }

    dna.descriptor.rotationUnit = static_cast<std::uint16_t>(rotationUnit);
}

void DNACalibDNAReaderImpl::unload(DataLayer layer) {
    if ((layer == DataLayer::All) || (layer == DataLayer::Descriptor)) {
        dna = DNA{dna.layers.unknownPolicy, dna.layers.upgradePolicy, memRes};
    } else if (layer == DataLayer::TwistSwingBehavior) {
        dna.unloadTwistSwingBehavior();
    } else if (layer == DataLayer::RBFBehavior) {
        dna.unloadRBFBehavior();
    } else if (layer == DataLayer::JointBehaviorMetadata) {
        dna.unloadJointBehaviorMetadata();
    } else if (layer == DataLayer::MachineLearnedBehavior) {
        dna.unloadMachineLearnedBehavior();
    } else if ((layer == DataLayer::Geometry) || (layer == DataLayer::GeometryWithoutBlendShapes)) {
        dna.unloadGeometry();
    } else if (layer == DataLayer::Behavior) {
        dna.unloadRBFBehavior();
        dna.unloadBehavior();
    } else if (layer == DataLayer::Definition) {
        dna.unloadJointBehaviorMetadata();
        dna.unloadTwistSwingBehavior();
        dna.unloadRBFBehavior();
        dna.unloadMachineLearnedBehavior();
        dna.unloadGeometry();
        dna.unloadBehavior();
        dna.unloadDefinition();
    }
}

}  // namespace dnac
