// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/DNACalibDNAReader.h"
#include "dnacalib/dna/ReaderImpl.h"
#include "dnacalib/dna/WriterImpl.h"

#include <dna/Reader.h>
#include <dna/Writer.h>

#include <cstdint>

namespace dnac {

class DNACalibDNAReaderImpl : public ReaderImpl<DNACalibDNAReader>, public WriterImpl<dna::Writer> {
public:
    explicit DNACalibDNAReaderImpl(MemoryResource* memRes_);

    virtual ~DNACalibDNAReaderImpl();

    void unload(DataLayer layer) override;

    DNACalibDNAReaderImpl(const DNACalibDNAReaderImpl&) = delete;
    DNACalibDNAReaderImpl& operator=(const DNACalibDNAReaderImpl&) = delete;

    DNACalibDNAReaderImpl(DNACalibDNAReaderImpl&&) = delete;
    DNACalibDNAReaderImpl& operator=(DNACalibDNAReaderImpl&&) = delete;

    using WriterImpl<dna::Writer>::setNeutralJointTranslations;
    void setNeutralJointTranslations(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs);
    void setNeutralJointTranslations(RawVector3Vector&& translations);
    void setNeutralJointTranslation(std::uint16_t index, const Vector3& translation);

    using WriterImpl<dna::Writer>::setNeutralJointRotations;
    void setNeutralJointRotations(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs);
    void setNeutralJointRotations(RawVector3Vector&& rotations);
    void setNeutralJointRotation(std::uint16_t index, const Vector3& rotation);

    using WriterImpl<dna::Writer>::setJointGroupValues;
    void setJointGroupValues(std::uint16_t jointGroupIndex, AlignedDynArray<float>&& values);

    using WriterImpl<dna::Writer>::setVertexPositions;
    void setVertexPositions(std::uint16_t meshIndex,
                            ConstArrayView<float> xs,
                            ConstArrayView<float> ys,
                            ConstArrayView<float> zs);
    void setVertexPositions(std::uint16_t meshIndex, RawVector3Vector&& positions);

    using WriterImpl<dna::Writer>::setBlendShapeTargetDeltas;
    void setBlendShapeTargetDeltas(std::uint16_t meshIndex,
                                   std::uint16_t blendShapeTargetIndex,
                                   ConstArrayView<float> xs,
                                   ConstArrayView<float> ys,
                                   ConstArrayView<float> zs);
    void setBlendShapeTargetDeltas(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex, RawVector3Vector&& deltas);

    using WriterImpl<dna::Writer>::setBlendShapeTargetVertexIndices;
    void setBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                          std::uint16_t blendShapeTargetIndex,
                                          ConstArrayView<std::uint32_t> vertexIndices);

    void pruneBlendShapeTargets(float threshold);

    void removeMeshes(ConstArrayView<std::uint16_t> meshIndices);
    void removeJoints(ConstArrayView<std::uint16_t> jointIndices);
    void removeJointAnimations(ConstArrayView<std::uint16_t> jointIndex);
    void removeBlendShapes(ConstArrayView<std::uint16_t> blendShapeIndices);
    void removeAnimatedMaps(ConstArrayView<std::uint16_t> animatedMapIndices);

    void convertToTranslationUnit(TranslationUnit translationUnit);
    void convertToRotationUnit(RotationUnit rotationUnit);
};

}  // namespace dnac
