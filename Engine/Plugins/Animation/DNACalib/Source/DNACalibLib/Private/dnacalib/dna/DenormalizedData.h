// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/dna/Aliases.h"
#include "dnacalib/dna/LODMapping.h"
#include "dnacalib/types/Aliases.h"
#include "dnacalib/utils/Extd.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cmath>
#include <cstdint>
#include <limits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace dnac {

template<class Reader>
struct DenormalizedData {
    LODMapping jointVariableAttributeIndices;
    LODMapping meshBlendShapeMappingIndices;
    Matrix<std::uint16_t> rbfPoseJointOutputIndices;
    Matrix<float> rbfPoseJointOutputValues;
    Matrix<std::uint16_t> rbfBlendShapeChannelOutputIndices;
    Matrix<std::uint16_t> rbfAnimatedMapOutputIndices;
    Vector<Matrix<std::uint32_t>> mlOperationParameters;
    Vector<Matrix<std::uint16_t>> mlDependencyOperationSetIndices;
    Vector<Matrix<std::uint16_t>> mlDependencyOperationIndices;

    explicit DenormalizedData(MemoryResource* memRes) :
        jointVariableAttributeIndices{memRes},
        meshBlendShapeMappingIndices{memRes},
        rbfPoseJointOutputIndices{memRes},
        rbfPoseJointOutputValues{memRes},
        rbfBlendShapeChannelOutputIndices{memRes},
        rbfAnimatedMapOutputIndices{memRes},
        mlOperationParameters{memRes},
        mlDependencyOperationSetIndices{memRes},
        mlDependencyOperationIndices{memRes} {
    }

public:
    void populate(const Reader* source) {
        populateJointVariableAttributeIndices(source);
        populateMeshBlendShapeMappingIndices(source);
        populateRBFPoseJointOutputIndices(source);
        populateRBFPoseJointOutputValues(source);
        populateRBFBlendShapeOutputIndices(source);
        populateRBFAnimatedMapOutputIndices(source);
        populateMLOperationParameters(source);
        populateMLDependencyOperationSetIndices(source);
        populateMLDependencyOperationIndices(source);
    }

    void populateJointVariableAttributeIndices(const Reader* source) {
        // Prepare storage for all available LODs
        const auto lodCount = source->getLODCount();
        jointVariableAttributeIndices.setLODCount(lodCount);
        auto memRes = rbfPoseJointOutputIndices.get_allocator().getMemoryResource();
        UnorderedSet<std::uint16_t> twswAttributeIndicesSet{memRes};

        for (std::uint16_t i = {}; i < source->getTwistCount(); ++i) {
            auto jointIndices = source->getTwistOutputJointIndices(i);
            for (const auto jointIndex : jointIndices) {
                twswAttributeIndicesSet.emplace(static_cast<std::uint16_t>(jointIndex * 9u + 3u));
                twswAttributeIndicesSet.emplace(static_cast<std::uint16_t>(jointIndex * 9u + 4u));
                twswAttributeIndicesSet.emplace(static_cast<std::uint16_t>(jointIndex * 9u + 5u));
            }
        }
        for (std::uint16_t i = {}; i < source->getSwingCount(); ++i) {
            auto jointIndices = source->getSwingOutputJointIndices(i);
            for (const auto jointIndex : jointIndices) {
                twswAttributeIndicesSet.emplace(static_cast<std::uint16_t>(jointIndex * 9u + 3u));
                twswAttributeIndicesSet.emplace(static_cast<std::uint16_t>(jointIndex * 9u + 4u));
                twswAttributeIndicesSet.emplace(static_cast<std::uint16_t>(jointIndex * 9u + 5u));
            }
        }

        Vector<UnorderedSet<std::uint16_t>> mlAttributeIndicesSet{memRes};
        mlAttributeIndicesSet.resize(lodCount);

        auto isJointInLOD = [source](std::uint16_t jointIndex, std::uint16_t lod) {
            const auto jointsInLOD = source->getJointIndicesForLOD(lod);
            return std::find(jointsInLOD.begin(), jointsInLOD.end(), jointIndex) != jointsInLOD.end();
        };

        auto getNumMLAttributesPerJoint = [source]() {
            const auto paramKeys = source->getMLJointsParameterKeys();
            const auto paramValues = source->getMLJointsParameterValues();
            std::uint32_t numMLAttributesPerJoint = 0u;
            for (std::size_t i = {}; i < paramKeys.size(); ++i) {
                if (paramKeys[i] == static_cast<std::uint16_t>(MachineLearnedBehaviorParameterKey::JointTranslationType)) {
                    const auto translationType = static_cast<TranslationRepresentation>(paramValues[i]);
                    if (translationType == TranslationRepresentation::Vector) {
                        numMLAttributesPerJoint += 3u;
                    }
                }
                if (paramKeys[i] == static_cast<std::uint16_t>(MachineLearnedBehaviorParameterKey::JointRotationType)) {
                    const auto rotationType = static_cast<RotationRepresentation>(paramValues[i]);
                    if (rotationType == RotationRepresentation::EulerAngles) {
                        numMLAttributesPerJoint += 3u;
                    } else if (rotationType == RotationRepresentation::Quaternion) {
                        numMLAttributesPerJoint += 4u;
                    }
                }
                if (paramKeys[i] == static_cast<std::uint16_t>(MachineLearnedBehaviorParameterKey::JointScaleType)) {
                    const auto scaleType = static_cast<ScaleRepresentation>(paramValues[i]);
                    if (scaleType == ScaleRepresentation::Vector) {
                        numMLAttributesPerJoint += 3u;
                    }
                }
            }
            return static_cast<std::uint16_t>(numMLAttributesPerJoint);
        };

        const auto numAttributesPerJoint = 9u;
        const auto numMLAttributesPerJoint = getNumMLAttributesPerJoint();
        const auto delta = static_cast<std::int32_t>(numAttributesPerJoint) - static_cast<std::int32_t>(numMLAttributesPerJoint);
        const auto mlJointsOutputIndices = source->getMLJointsOutputIndices();
        for (std::uint16_t i = 0; i < mlJointsOutputIndices.size(); ++i) {
            const auto outputAttrIndex = mlJointsOutputIndices[i];
            // NOTE: This presumes that we always have all attributes for ML outputs
            const auto jointIndex = static_cast<std::uint16_t>(outputAttrIndex / numMLAttributesPerJoint);
            const auto newAttrBase = static_cast<std::uint16_t>(jointIndex * numAttributesPerJoint);
            const auto relAttrIndex = static_cast<std::uint16_t>(outputAttrIndex % numMLAttributesPerJoint);
            // Skip qw
            if (relAttrIndex == 6u) {
                continue;
            }
            const auto newRelAttrIndex = (relAttrIndex < 6u ? relAttrIndex : static_cast<std::uint16_t>(delta + relAttrIndex));
            std::uint16_t remappedOutputIndex = static_cast<std::uint16_t>(newAttrBase + newRelAttrIndex);

            for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
                if (isJointInLOD(jointIndex, lod)) {
                    mlAttributeIndicesSet[lod].emplace(remappedOutputIndex);
                }
            }
        }

        Vector<UnorderedSet<std::uint16_t>> distinctIndicesPerLOD{lodCount, twswAttributeIndicesSet, memRes};

        // Add ML output indices
        for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
            distinctIndicesPerLOD[lod].insert(mlAttributeIndicesSet[lod].begin(), mlAttributeIndicesSet[lod].end());
        }

        // Concatenate all output indices for each LOD
        for (std::uint16_t i = {}; i < source->getJointGroupCount(); ++i) {
            const auto outputIndices = source->getJointGroupOutputIndices(i);
            const auto lodSizes = source->getJointGroupLODs(i);
            assert(lodSizes.size() == lodCount);
            for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
                // In this case, each LOD has a distinct set of indices, so the LOD and Index parameters
                // are the same for all LODs
                for (std::uint16_t oi = {}; oi < lodSizes[lod]; ++oi) {
                    distinctIndicesPerLOD[lod].emplace(outputIndices[oi]);
                }
            }
        }

        for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
            jointVariableAttributeIndices.addIndices(lod, distinctIndicesPerLOD[lod].begin(), distinctIndicesPerLOD[lod].end());
            jointVariableAttributeIndices.associateLODWithIndices(lod, lod);
        }
        jointVariableAttributeIndices.sortIndices();
    }

    void populateMeshBlendShapeMappingIndices(const Reader* source) {
        // Prepare storage for all available LODs
        const auto lodCount = source->getLODCount();
        meshBlendShapeMappingIndices.setLODCount(lodCount);
        // Include only those mapping indices which are present in the already filtered
        // mesh and blendshape LOD mapping
        for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
            const auto meshIndices = source->getMeshIndicesForLOD(lod);
            const auto blendShapeIndices = source->getBlendShapeChannelIndicesForLOD(lod);

            auto isMappingNeeded = [&meshIndices, &blendShapeIndices](const MeshBlendShapeChannelMapping& mapping) {
                const bool meshNeeded = extd::contains(meshIndices, mapping.meshIndex);
                const bool blendShapeNeeded = extd::contains(blendShapeIndices, mapping.blendShapeChannelIndex);
                return (meshNeeded && blendShapeNeeded);
            };

            for (std::uint16_t i = {}; i < source->getMeshBlendShapeChannelMappingCount(); ++i) {
                const auto mapping = source->getMeshBlendShapeChannelMapping(i);
                if (isMappingNeeded(mapping)) {
                    // In this case, each LOD has a distinct set of indices, so the LOD and Index parameters
                    // are the same for all LODs
                    meshBlendShapeMappingIndices.addIndices(lod, &i, static_cast<std::uint16_t>(1));
                }
            }

            meshBlendShapeMappingIndices.associateLODWithIndices(lod, lod);
        }
    }

    void populateRBFPoseJointOutputIndices(const Reader* source) {
        const std::uint16_t poseCount = source->getRBFPoseCount();
        rbfPoseJointOutputIndices.resize(poseCount);
        for (std::uint16_t pi = {}; pi < poseCount; ++pi) {
            const auto poseControlIndices = source->getRBFPoseOutputControlIndices(pi);
            for (std::uint16_t gi = {}; gi < source->getJointGroupCount(); ++gi) {
                const auto inputIndices = source->getJointGroupInputIndices(gi);
                const auto outputIndices = source->getJointGroupOutputIndices(gi);
                const auto values = source->getJointGroupValues(gi);
                const auto columnCount = inputIndices.size();
                const auto rowCount = outputIndices.size();
                for (std::size_t columnIndex = {}; columnIndex < columnCount; ++columnIndex) {
                    const auto inputIndex = inputIndices[columnIndex];
                    if (std::find(poseControlIndices.begin(), poseControlIndices.end(), inputIndex) != poseControlIndices.end()) {
                        rbfPoseJointOutputIndices[pi].reserve(rbfPoseJointOutputIndices[pi].size() + rowCount);
                        for (std::uint16_t rowIndex = {}; rowIndex < rowCount; ++rowIndex) {
                            if (std::abs(values[rowIndex * columnCount + columnIndex]) > 0.0f) {
                                rbfPoseJointOutputIndices[pi].push_back(outputIndices[rowIndex]);
                            }
                        }
                    }
                }
            }
        }
    }

    void populateRBFPoseJointOutputValues(const Reader* source) {
        const std::uint16_t poseCount = source->getRBFPoseCount();
        rbfPoseJointOutputValues.resize(poseCount);

        auto memRes = rbfPoseJointOutputValues.get_allocator().getMemoryResource();
        auto jointIndexToJointGroupMap = [&]() {
            Vector<std::uint16_t> map{source->getJointCount(), {}, memRes};
            const auto groupCount = source->getJointGroupCount();
            for (std::uint16_t gi = {}; gi < groupCount; ++gi) {
                const auto jointIndices = source->getJointGroupJointIndices(gi);
                for (auto ji : jointIndices) {
                    map[ji] = gi;
                }
            }
            return map;
        }();

        const auto indexOf = [](ConstArrayView<std::uint16_t> container, std::uint16_t val) {
            const auto valIt = std::find(container.begin(), container.end(), val);
            return static_cast<size_t>(std::distance(container.begin(), valIt));
        };

        for (std::uint16_t pi = {}; pi < poseCount; ++pi) {
            const auto poseControlIndices = source->getRBFPoseOutputControlIndices(pi);
            for (const auto inputIndex : poseControlIndices) {
                const auto jointOutputIndices = source->getRBFPoseJointOutputIndices(pi);
                const auto jointOutputIndicesCount = jointOutputIndices.size();
                rbfPoseJointOutputValues[pi].resize(jointOutputIndices.size(), 0.0f);
                for (std::uint16_t oi = {}; oi < jointOutputIndicesCount; ++oi) {
                    const auto outputIndex = jointOutputIndices[oi];
                    const auto jointIndex = static_cast<std::uint16_t>(outputIndex / 9);
                    const auto jointGroupIndex = jointIndexToJointGroupMap[jointIndex];

                    const auto jointGroupOutputIndices = source->getJointGroupOutputIndices(jointGroupIndex);
                    const auto rowCount = jointGroupOutputIndices.size();

                    const auto jointGroupInputIndices = source->getJointGroupInputIndices(jointGroupIndex);
                    const auto columnCount = jointGroupInputIndices.size();

                    const auto columnIndex = indexOf(jointGroupInputIndices, inputIndex);
                    const auto rowIndex = indexOf(jointGroupOutputIndices, outputIndex);

                    if ((columnIndex < columnCount) && (rowIndex < rowCount)) {
                        const auto values = source->getJointGroupValues(jointGroupIndex);
                        rbfPoseJointOutputValues[pi][oi] = values[rowIndex * columnCount + columnIndex];
                    }
                }
            }
        }
    }

    void populateRBFBlendShapeOutputIndices(const Reader* source) {
        const std::uint16_t solverCount = source->getRBFSolverCount();
        rbfBlendShapeChannelOutputIndices.resize(solverCount);
        const auto bscInputIndices = source->getBlendShapeChannelInputIndices();
        const auto bscOutputIndices = source->getBlendShapeChannelOutputIndices();
        const auto bscCount = bscInputIndices.size();
        const auto poseCount = source->getRBFPoseCount();
        rbfBlendShapeChannelOutputIndices.resize(poseCount);
        for (std::uint16_t pi = {}; pi < poseCount; ++pi) {
            const auto poseControlIndices = source->getRBFPoseOutputControlIndices(pi);
            for (const auto inputIndex : poseControlIndices) {
                for (std::uint16_t bi = {}; bi < bscCount; ++bi) {
                    if (bscInputIndices[bi] == inputIndex) {
                        rbfBlendShapeChannelOutputIndices[pi].push_back(bscOutputIndices[bi]);
                    }
                }
            }
        }
    }

    void populateRBFAnimatedMapOutputIndices(const Reader* source) {
        const std::uint16_t solverCount = source->getRBFSolverCount();
        rbfAnimatedMapOutputIndices.resize(solverCount);
        const auto amInputIndices = source->getAnimatedMapInputIndices();
        const auto amOutputIndices = source->getAnimatedMapOutputIndices();
        const auto amCount = source->getAnimatedMapCount();
        const auto poseCount = source->getRBFPoseCount();
        for (std::uint16_t pi = {}; pi < poseCount; ++pi) {
            const auto poseControlIndices = source->getRBFPoseOutputControlIndices(pi);
            for (const auto inputIndex : poseControlIndices) {
                for (std::uint16_t ai = {}; ai < amCount; ++ai) {
                    if (amInputIndices[ai] == inputIndex) {
                        rbfAnimatedMapOutputIndices[pi].push_back(amOutputIndices[ai]);
                    }
                }
            }
        }
    }

    void populateMLOperationParameters(const Reader* source) {
        const auto neuralNetCount = source->getNeuralNetworkCount();
        mlOperationParameters.resize(3);
        mlOperationParameters[0].resize(neuralNetCount);
        mlOperationParameters[1].resize(neuralNetCount);
        mlOperationParameters[2].resize(neuralNetCount);
        for (std::uint16_t i = {}; i < neuralNetCount; ++i) {
            const auto inputIndices = source->getNeuralNetworkInputIndices(i);
            const auto outputIndices = source->getNeuralNetworkOutputIndices(i);
            mlOperationParameters[0][i].assign(inputIndices.begin(), inputIndices.end());
            mlOperationParameters[1][i].push_back(i);
            mlOperationParameters[2][i].assign(outputIndices.begin(), outputIndices.end());
        }
    }

    void populateMLDependencyOperationSetIndices(const Reader* source) {
        const auto neuralNetCount = source->getNeuralNetworkCount();
        mlDependencyOperationSetIndices.resize(3);
        // Operation 0 has no dependencies
        mlDependencyOperationSetIndices[1].resize(neuralNetCount);
        mlDependencyOperationSetIndices[2].resize(neuralNetCount);
        for (std::uint16_t i = {}; i < neuralNetCount; ++i) {
            mlDependencyOperationSetIndices[1][i].push_back(0);
            mlDependencyOperationSetIndices[2][i].push_back(1);
        }
    }

    void populateMLDependencyOperationIndices(const Reader* source) {
        const auto neuralNetCount = source->getNeuralNetworkCount();
        mlDependencyOperationIndices.resize(3);
        // Operation 0 has no dependencies
        mlDependencyOperationIndices[1].resize(neuralNetCount);
        mlDependencyOperationIndices[2].resize(neuralNetCount);
        for (std::uint16_t i = {}; i < neuralNetCount; ++i) {
            mlDependencyOperationIndices[1][i].push_back(i);
            mlDependencyOperationIndices[2][i].push_back(i);
        }
    }
};

}  // namespace dnac
