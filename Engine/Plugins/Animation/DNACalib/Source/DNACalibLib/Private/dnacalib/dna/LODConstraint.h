// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"

#include <cstdint>

namespace dnac {

class LODConstraint {
public:
    LODConstraint(std::uint16_t maxLOD, std::uint16_t minLOD, MemoryResource* memRes);
    LODConstraint(ConstArrayView<std::uint16_t> lods, MemoryResource* memRes);

    bool hasImpactOn(std::uint16_t lodCount) const;
    std::uint16_t getMaxLOD() const;
    std::uint16_t getMinLOD() const;
    std::uint16_t getLODCount() const;
    void clampTo(std::uint16_t lodCount);
    void applyTo(Vector<std::uint16_t>& unconstrainedLODs) const;
    void applyTo(DynArray<std::uint16_t>& unconstrainedLODs) const;

private:
    Vector<std::uint16_t> lods;
};

}  // namespace dnac
