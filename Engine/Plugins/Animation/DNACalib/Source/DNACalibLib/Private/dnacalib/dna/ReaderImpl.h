// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/Aliases.h"
#include "dnacalib/dna/BaseImpl.h"
#include "dnacalib/dna/DenormalizedData.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstddef>
#include <limits>
#include <tuple>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace dnac {

template<class TReaderBase>
class ReaderImpl : public TReaderBase, public virtual BaseImpl {
public:
    explicit ReaderImpl(MemoryResource* memRes_);

    // HeaderReader methods
    std::uint16_t getFileFormatGeneration() const override;
    std::uint16_t getFileFormatVersion() const override;

    // DescriptorReader methods
    StringView getName() const override;
    Archetype getArchetype() const override;
    Gender getGender() const override;
    std::uint16_t getAge() const override;
    std::uint32_t getMetaDataCount() const override;
    StringView getMetaDataKey(std::uint32_t index) const override;
    StringView getMetaDataValue(const char* key) const override;
    TranslationUnit getTranslationUnit() const override;
    RotationUnit getRotationUnit() const override;
    CoordinateSystem getCoordinateSystem() const override;
    RotationSequence getRotationSequence() const override;
    RotationSign getRotationSign() const override;
    FaceWindingOrder getFaceWindingOrder() const override;
    std::uint16_t getLODCount() const override;
    std::uint16_t getDBMaxLOD() const override;
    StringView getDBComplexity() const override;
    StringView getDBName() const override;

    // DefinitionReader methods
    std::uint16_t getGUIControlCount() const override;
    StringView getGUIControlName(std::uint16_t index) const override;
    std::uint16_t getRawControlCount() const override;
    StringView getRawControlName(std::uint16_t index) const override;
    std::uint16_t getJointCount() const override;
    StringView getJointName(std::uint16_t index) const override;
    std::uint16_t getJointIndexListCount() const override;
    ConstArrayView<std::uint16_t> getJointIndicesForLOD(std::uint16_t lod) const override;
    std::uint16_t getJointParentIndex(std::uint16_t index) const override;
    std::uint16_t getBlendShapeChannelCount() const override;
    StringView getBlendShapeChannelName(std::uint16_t index) const override;
    std::uint16_t getBlendShapeChannelIndexListCount() const override;
    ConstArrayView<std::uint16_t> getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const override;
    std::uint16_t getAnimatedMapCount() const override;
    StringView getAnimatedMapName(std::uint16_t index) const override;
    std::uint16_t getAnimatedMapIndexListCount() const override;
    ConstArrayView<std::uint16_t> getAnimatedMapIndicesForLOD(std::uint16_t lod) const override;
    std::uint16_t getMeshCount() const override;
    StringView getMeshName(std::uint16_t index) const override;
    std::uint16_t getMeshIndexListCount() const override;
    ConstArrayView<std::uint16_t> getMeshIndicesForLOD(std::uint16_t lod) const override;
    std::uint16_t getMeshBlendShapeChannelMappingCount() const override;
    MeshBlendShapeChannelMapping getMeshBlendShapeChannelMapping(std::uint16_t index) const override;
    ConstArrayView<std::uint16_t> getMeshBlendShapeChannelMappingIndicesForLOD(std::uint16_t lod) const override;
    Vector3 getNeutralJointTranslation(std::uint16_t index) const override;
    ConstArrayView<float> getNeutralJointTranslationXs() const override;
    ConstArrayView<float> getNeutralJointTranslationYs() const override;
    ConstArrayView<float> getNeutralJointTranslationZs() const override;
    Vector3 getNeutralJointRotation(std::uint16_t index) const override;
    ConstArrayView<float> getNeutralJointRotationXs() const override;
    ConstArrayView<float> getNeutralJointRotationYs() const override;
    ConstArrayView<float> getNeutralJointRotationZs() const override;

    // BehaviorReader methods
    ConstArrayView<std::uint16_t> getGUIToRawInputIndices() const override;
    ConstArrayView<std::uint16_t> getGUIToRawOutputIndices() const override;
    ConstArrayView<float> getGUIToRawFromValues() const override;
    ConstArrayView<float> getGUIToRawToValues() const override;
    ConstArrayView<float> getGUIToRawSlopeValues() const override;
    ConstArrayView<float> getGUIToRawCutValues() const override;
    std::uint16_t getPSDCount() const override;
    ConstArrayView<std::uint16_t> getPSDRowIndices() const override;
    ConstArrayView<std::uint16_t> getPSDColumnIndices() const override;
    ConstArrayView<float> getPSDValues() const override;
    std::uint16_t getJointRowCount() const override;
    std::uint16_t getJointColumnCount() const override;
    ConstArrayView<std::uint16_t> getJointVariableAttributeIndices(std::uint16_t lod) const override;
    std::uint16_t getJointGroupCount() const override;
    ConstArrayView<std::uint16_t> getJointGroupLODs(std::uint16_t jointGroupIndex) const override;
    ConstArrayView<std::uint16_t> getJointGroupInputIndices(std::uint16_t jointGroupIndex) const override;
    ConstArrayView<std::uint16_t> getJointGroupOutputIndices(std::uint16_t jointGroupIndex) const override;
    ConstArrayView<float> getJointGroupValues(std::uint16_t jointGroupIndex) const override;
    ConstArrayView<std::uint16_t> getJointGroupJointIndices(std::uint16_t jointGroupIndex) const override;
    ConstArrayView<std::uint16_t> getBlendShapeChannelLODs() const override;
    ConstArrayView<std::uint16_t> getBlendShapeChannelOutputIndices() const override;
    ConstArrayView<std::uint16_t> getBlendShapeChannelInputIndices() const override;
    ConstArrayView<std::uint16_t> getAnimatedMapLODs() const override;
    ConstArrayView<std::uint16_t> getAnimatedMapInputIndices() const override;
    ConstArrayView<std::uint16_t> getAnimatedMapOutputIndices() const override;
    ConstArrayView<float> getAnimatedMapFromValues() const override;
    ConstArrayView<float> getAnimatedMapToValues() const override;
    ConstArrayView<float> getAnimatedMapSlopeValues() const override;
    ConstArrayView<float> getAnimatedMapCutValues() const override;

    // GeometryReader methods
    std::uint32_t getVertexPositionCount(std::uint16_t meshIndex) const override;
    Position getVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override;
    ConstArrayView<float> getVertexPositionXs(std::uint16_t meshIndex) const override;
    ConstArrayView<float> getVertexPositionYs(std::uint16_t meshIndex) const override;
    ConstArrayView<float> getVertexPositionZs(std::uint16_t meshIndex) const override;
    std::uint32_t getVertexTextureCoordinateCount(std::uint16_t meshIndex) const override;
    TextureCoordinate getVertexTextureCoordinate(std::uint16_t meshIndex, std::uint32_t textureCoordinateIndex) const override;
    ConstArrayView<float> getVertexTextureCoordinateUs(std::uint16_t meshIndex) const override;
    ConstArrayView<float> getVertexTextureCoordinateVs(std::uint16_t meshIndex) const override;
    std::uint32_t getVertexNormalCount(std::uint16_t meshIndex) const override;
    Normal getVertexNormal(std::uint16_t meshIndex, std::uint32_t normalIndex) const override;
    ConstArrayView<float> getVertexNormalXs(std::uint16_t meshIndex) const override;
    ConstArrayView<float> getVertexNormalYs(std::uint16_t meshIndex) const override;
    ConstArrayView<float> getVertexNormalZs(std::uint16_t meshIndex) const override;
    std::uint32_t getFaceCount(std::uint16_t meshIndex) const override;
    ConstArrayView<std::uint32_t> getFaceVertexLayoutIndices(std::uint16_t meshIndex, std::uint32_t faceIndex) const override;
    std::uint32_t getVertexLayoutCount(std::uint16_t meshIndex) const override;
    VertexLayout getVertexLayout(std::uint16_t meshIndex, std::uint32_t layoutIndex) const override;
    ConstArrayView<std::uint32_t> getVertexLayoutPositionIndices(std::uint16_t meshIndex) const override;
    ConstArrayView<std::uint32_t> getVertexLayoutTextureCoordinateIndices(std::uint16_t meshIndex) const override;
    ConstArrayView<std::uint32_t> getVertexLayoutNormalIndices(std::uint16_t meshIndex) const override;
    std::uint16_t getMaximumInfluencePerVertex(std::uint16_t meshIndex) const override;
    std::uint32_t getSkinWeightsCount(std::uint16_t meshIndex) const override;
    ConstArrayView<float> getSkinWeightsValues(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override;
    ConstArrayView<std::uint16_t> getSkinWeightsJointIndices(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override;
    std::uint16_t getBlendShapeTargetCount(std::uint16_t meshIndex) const override;
    std::uint16_t getBlendShapeChannelIndex(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override;
    std::uint32_t getBlendShapeTargetDeltaCount(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override;
    Delta getBlendShapeTargetDelta(std::uint16_t meshIndex,
                                   std::uint16_t blendShapeTargetIndex,
                                   std::uint32_t deltaIndex) const override;
    ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override;
    ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override;
    ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override;
    ConstArrayView<std::uint32_t> getBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                                   std::uint16_t blendShapeTargetIndex) const override;
    // MachineLearnedBehaviorReader methods
    std::uint16_t getMLControlCount() const override;
    StringView getMLControlName(std::uint16_t index) const override;
    std::uint16_t getNeuralNetworkCount() const override;
    std::uint16_t getNeuralNetworkIndexListCount() const override;
    ConstArrayView<std::uint16_t> getNeuralNetworkIndicesForLOD(std::uint16_t lod) const override;
    std::uint16_t getMeshRegionCount(std::uint16_t meshIndex) const override;
    StringView getMeshRegionName(std::uint16_t meshIndex, std::uint16_t regionIndex) const override;
    ConstArrayView<std::uint16_t> getNeuralNetworkIndicesForMeshRegion(std::uint16_t meshIndex,
                                                                       std::uint16_t regionIndex) const override;
    ConstArrayView<std::uint16_t> getNeuralNetworkInputIndices(std::uint16_t netIndex) const override;
    ConstArrayView<std::uint16_t> getNeuralNetworkOutputIndices(std::uint16_t netIndex) const override;
    std::uint16_t getNeuralNetworkLayerCount(std::uint16_t netIndex) const override;
    ActivationFunction getNeuralNetworkLayerActivationFunction(std::uint16_t netIndex, std::uint16_t layerIndex) const override;
    ConstArrayView<float> getNeuralNetworkLayerActivationFunctionParameters(std::uint16_t netIndex,
                                                                            std::uint16_t layerIndex) const override;
    ConstArrayView<float> getNeuralNetworkLayerBiases(std::uint16_t netIndex, std::uint16_t layerIndex) const override;
    ConstArrayView<float> getNeuralNetworkLayerWeights(std::uint16_t netIndex, std::uint16_t layerIndex) const override;

    // RBFBehaviorReader methods
    std::uint16_t getRBFPoseCount() const override;
    StringView getRBFPoseName(std::uint16_t index) const override;
    ConstArrayView<std::uint16_t> getRBFPoseJointOutputIndices(std::uint16_t poseIndex) const override;
    ConstArrayView<std::uint16_t> getRBFPoseBlendShapeChannelOutputIndices(std::uint16_t poseIndex) const override;
    ConstArrayView<std::uint16_t> getRBFPoseAnimatedMapOutputIndices(std::uint16_t poseIndex) const override;
    ConstArrayView<float> getRBFPoseJointOutputValues(std::uint16_t poseIndex) const override;
    float getRBFPoseScale(std::uint16_t poseIndex) const override;
    std::uint16_t getRBFPoseControlCount() const override;
    StringView getRBFPoseControlName(std::uint16_t poseControlIndex) const override;
    ConstArrayView<std::uint16_t> getRBFPoseInputControlIndices(std::uint16_t poseIndex) const override;
    ConstArrayView<std::uint16_t> getRBFPoseOutputControlIndices(std::uint16_t poseIndex) const override;
    ConstArrayView<float> getRBFPoseOutputControlWeights(std::uint16_t poseIndex) const override;
    std::uint16_t getRBFSolverCount() const override;
    std::uint16_t getRBFSolverIndexListCount() const override;
    ConstArrayView<std::uint16_t> getRBFSolverIndicesForLOD(std::uint16_t lod) const override;
    StringView getRBFSolverName(std::uint16_t index) const override;
    ConstArrayView<std::uint16_t> getRBFSolverRawControlIndices(std::uint16_t solverIndex) const override;
    ConstArrayView<std::uint16_t> getRBFSolverPoseIndices(std::uint16_t solverIndex) const override;
    ConstArrayView<float> getRBFSolverRawControlValues(std::uint16_t solverIndex) const override;
    RBFSolverType getRBFSolverType(std::uint16_t solverIndex) const override;
    float getRBFSolverRadius(std::uint16_t solverIndex) const override;
    AutomaticRadius getRBFSolverAutomaticRadius(std::uint16_t solverIndex) const override;
    float getRBFSolverWeightThreshold(std::uint16_t solverIndex) const override;
    RBFDistanceMethod getRBFSolverDistanceMethod(std::uint16_t solverIndex) const override;
    RBFNormalizeMethod getRBFSolverNormalizeMethod(std::uint16_t solverIndex) const override;
    RBFFunctionType getRBFSolverFunctionType(std::uint16_t solverIndex) const override;
    TwistAxis getRBFSolverTwistAxis(std::uint16_t solverIndex) const override;

    // JointBehaviorMetadataReader methods
    TranslationRepresentation getJointTranslationRepresentation(std::uint16_t jointIndex) const override;
    RotationRepresentation getJointRotationRepresentation(std::uint16_t jointIndex) const override;
    ScaleRepresentation getJointScaleRepresentation(std::uint16_t jointIndex) const override;

    // TwistSwingBehaviorReader methods
    std::uint16_t getTwistCount() const override;
    TwistAxis getTwistSetupTwistAxis(std::uint16_t twistIndex) const override;
    ConstArrayView<std::uint16_t> getTwistInputControlIndices(std::uint16_t twistIndex) const override;
    ConstArrayView<std::uint16_t> getTwistOutputJointIndices(std::uint16_t twistIndex) const override;
    ConstArrayView<float> getTwistBlendWeights(std::uint16_t twistIndex) const override;
    std::uint16_t getSwingCount() const override;
    TwistAxis getSwingSetupTwistAxis(std::uint16_t swingIndex) const override;
    ConstArrayView<std::uint16_t> getSwingInputControlIndices(std::uint16_t swingIndex) const override;
    ConstArrayView<std::uint16_t> getSwingOutputJointIndices(std::uint16_t swingIndex) const override;
    ConstArrayView<float> getSwingBlendWeights(std::uint16_t swingIndex) const override;

    // MachineLearnedBehaviorExtReader methods
    std::uint16_t getMLTypeCount() const override;
    std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const override;
    std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override;
    MachineLearnedBehaviorOperationType getMLOperationType(std::uint16_t mlTypeIndex,
                                                           std::uint16_t mlOperationSetIndex,
                                                           std::uint16_t mlOperationIndex) const override;
    ConstArrayView<std::uint32_t> getMLOperationParameters(std::uint16_t mlTypeIndex,
                                                           std::uint16_t mlOperationSetIndex,
                                                           std::uint16_t mlOperationIndex) const override;
    ConstArrayView<std::uint16_t> getMLOperationDependencyOperationSetIndices(std::uint16_t mlTypeIndex,
                                                                              std::uint16_t mlOperationSetIndex,
                                                                              std::uint16_t mlOperationIndex) const override;
    ConstArrayView<std::uint16_t> getMLOperationDependencyOperationIndices(std::uint16_t mlTypeIndex,
                                                                           std::uint16_t mlOperationSetIndex,
                                                                           std::uint16_t mlOperationIndex) const override;
    std::uint16_t getMLOperationIndexListCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override;
    ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t mlTypeIndex,
                                                              std::uint16_t mlOperationSetIndex,
                                                              std::uint16_t lod) const override;
    ConstArrayView<std::uint16_t> getMLJointsInputIndices() const override;
    ConstArrayView<std::uint16_t> getMLJointsOutputIndices() const override;
    ConstArrayView<std::uint16_t> getMLJointsParameterKeys() const override;
    ConstArrayView<std::uint16_t> getMLJointsParameterValues() const override;

protected:
    mutable DenormalizedData<TReaderBase> cache;

private:
    bool isMLExtLoaded() const;
};

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4589)
#endif
template<class TReaderBase>
inline ReaderImpl<TReaderBase>::ReaderImpl(MemoryResource* memRes_) :
    BaseImpl{memRes_},
    cache{memRes_} {
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4505)
#endif
template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getFileFormatGeneration() const {
    return dna.version.generation;
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getFileFormatVersion() const {
    return dna.version.version;
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getName() const {
    return {dna.descriptor.name.data(), dna.descriptor.name.size()};
}

template<class TReaderBase>
inline Archetype ReaderImpl<TReaderBase>::getArchetype() const {
    return static_cast<Archetype>(dna.descriptor.archetype);
}

template<class TReaderBase>
inline Gender ReaderImpl<TReaderBase>::getGender() const {
    return static_cast<Gender>(dna.descriptor.gender);
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getAge() const {
    return dna.descriptor.age;
}

template<class TReaderBase>
inline std::uint32_t ReaderImpl<TReaderBase>::getMetaDataCount() const {
    return static_cast<std::uint32_t>(dna.descriptor.metadata.size());
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getMetaDataKey(std::uint32_t index) const {
    if (index < dna.descriptor.metadata.size()) {
        const auto& key = std::get<0>(dna.descriptor.metadata[index]);
        return {key.data(), key.size()};
    }
    return {};
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getMetaDataValue(const char* key) const {
    for (const auto& data : dna.descriptor.metadata) {
        if (std::get<0>(data) == key) {
            const auto& value = std::get<1>(data);
            return {value.data(), value.size()};
        }
    }
    return {};
}

template<class TReaderBase>
inline TranslationUnit ReaderImpl<TReaderBase>::getTranslationUnit() const {
    return static_cast<TranslationUnit>(dna.descriptor.translationUnit);
}

template<class TReaderBase>
inline RotationUnit ReaderImpl<TReaderBase>::getRotationUnit() const {
    return static_cast<RotationUnit>(dna.descriptor.rotationUnit);
}

template<class TReaderBase>
inline CoordinateSystem ReaderImpl<TReaderBase>::getCoordinateSystem() const {
    return {static_cast<Direction>(dna.descriptor.coordinateSystem.xAxis),
            static_cast<Direction>(dna.descriptor.coordinateSystem.yAxis),
            static_cast<Direction>(dna.descriptor.coordinateSystem.zAxis)};
}

template<class TReaderBase>
inline RotationSequence ReaderImpl<TReaderBase>::getRotationSequence() const {
    return dna.descriptorExt.rotationSequence;
}

template<class TReaderBase>
inline RotationSign ReaderImpl<TReaderBase>::getRotationSign() const {
    return dna.descriptorExt.rotationSign;
}

template<class TReaderBase>
inline FaceWindingOrder ReaderImpl<TReaderBase>::getFaceWindingOrder() const {
    return dna.descriptorExt.faceWindingOrder;
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getLODCount() const {
    return dna.descriptor.lodCount;
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getDBMaxLOD() const {
    return dna.descriptor.maxLOD;
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getDBComplexity() const {
    return {dna.descriptor.complexity.data(), dna.descriptor.complexity.size()};
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getDBName() const {
    return {dna.descriptor.dbName.data(), dna.descriptor.dbName.size()};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getGUIControlCount() const {
    return static_cast<std::uint16_t>(dna.definition.guiControlNames.size());
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getGUIControlName(std::uint16_t index) const {
    if (index < dna.definition.guiControlNames.size()) {
        const auto& guiControlName = dna.definition.guiControlNames[index];
        return {guiControlName.data(), guiControlName.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getRawControlCount() const {
    return static_cast<std::uint16_t>(dna.definition.rawControlNames.size());
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getRawControlName(std::uint16_t index) const {
    if (index < dna.definition.rawControlNames.size()) {
        const auto& rawControlName = dna.definition.rawControlNames[index];
        return {rawControlName.data(), rawControlName.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getJointCount() const {
    return static_cast<std::uint16_t>(dna.definition.jointNames.size());
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getJointName(std::uint16_t index) const {
    if (index < dna.definition.jointNames.size()) {
        const auto& jointName = dna.definition.jointNames[index];
        return {jointName.data(), jointName.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getJointIndexListCount() const {
    return dna.definition.lodJointMapping.getIndexListCount();
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getJointIndicesForLOD(std::uint16_t lod) const {
    return dna.definition.lodJointMapping.getIndices(lod);
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getJointParentIndex(std::uint16_t index) const {
    if (index < dna.definition.jointHierarchy.size()) {
        return dna.definition.jointHierarchy[index];
    }
    return std::numeric_limits<std::uint16_t>::max();
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getBlendShapeChannelCount() const {
    return static_cast<std::uint16_t>(dna.definition.blendShapeChannelNames.size());
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getBlendShapeChannelName(std::uint16_t index) const {
    if (index < dna.definition.blendShapeChannelNames.size()) {
        const auto& blendShapeName = dna.definition.blendShapeChannelNames[index];
        return {blendShapeName.data(), blendShapeName.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getBlendShapeChannelIndexListCount() const {
    return dna.definition.lodBlendShapeMapping.getIndexListCount();
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const {
    return dna.definition.lodBlendShapeMapping.getIndices(lod);
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getAnimatedMapCount() const {
    return static_cast<std::uint16_t>(dna.definition.animatedMapNames.size());
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getAnimatedMapName(std::uint16_t index) const {
    if (index < dna.definition.animatedMapNames.size()) {
        const auto& animatedMapName = dna.definition.animatedMapNames[index];
        return {animatedMapName.data(), animatedMapName.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getAnimatedMapIndexListCount() const {
    return dna.definition.lodAnimatedMapMapping.getIndexListCount();
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getAnimatedMapIndicesForLOD(std::uint16_t lod) const {
    return dna.definition.lodAnimatedMapMapping.getIndices(lod);
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getMeshCount() const {
    return static_cast<std::uint16_t>(dna.definition.meshNames.size());
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getMeshName(std::uint16_t index) const {
    if (index < dna.definition.meshNames.size()) {
        const auto& meshName = dna.definition.meshNames[index];
        return {meshName.data(), meshName.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getMeshIndexListCount() const {
    return dna.definition.lodMeshMapping.getIndexListCount();
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getMeshIndicesForLOD(std::uint16_t lod) const {
    return dna.definition.lodMeshMapping.getIndices(lod);
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getMeshBlendShapeChannelMappingCount() const {
    return static_cast<std::uint16_t>(dna.definition.meshBlendShapeChannelMapping.size());
}

template<class TReaderBase>
inline MeshBlendShapeChannelMapping ReaderImpl<TReaderBase>::getMeshBlendShapeChannelMapping(std::uint16_t index) const {
    const auto mapping = dna.definition.meshBlendShapeChannelMapping.get(index);
    return {mapping.from, mapping.to};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t>
ReaderImpl<TReaderBase>::getMeshBlendShapeChannelMappingIndicesForLOD(std::uint16_t lod) const {
    if (cache.meshBlendShapeMappingIndices.getLODCount() == static_cast<std::uint16_t>(0)) {
        cache.populateMeshBlendShapeMappingIndices(this);
    }
    return cache.meshBlendShapeMappingIndices.getIndices(lod);
}

template<class TReaderBase>
inline Vector3 ReaderImpl<TReaderBase>::getNeutralJointTranslation(std::uint16_t index) const {
    const auto& translations = dna.definition.neutralJointTranslations;
    if (index < translations.xs.size()) {
        return {translations.xs[index], translations.ys[index], translations.zs[index]};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getNeutralJointTranslationXs() const {
    const auto& xs = dna.definition.neutralJointTranslations.xs;
    return {xs.data(), xs.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getNeutralJointTranslationYs() const {
    const auto& ys = dna.definition.neutralJointTranslations.ys;
    return {ys.data(), ys.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getNeutralJointTranslationZs() const {
    const auto& zs = dna.definition.neutralJointTranslations.zs;
    return {zs.data(), zs.size()};
}

template<class TReaderBase>
inline Vector3 ReaderImpl<TReaderBase>::getNeutralJointRotation(std::uint16_t index) const {
    const auto& rotations = dna.definition.neutralJointRotations;
    if (index < rotations.size()) {
        return {rotations.xs[index], rotations.ys[index], rotations.zs[index]};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getNeutralJointRotationXs() const {
    const auto& xs = dna.definition.neutralJointRotations.xs;
    return {xs.data(), xs.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getNeutralJointRotationYs() const {
    const auto& ys = dna.definition.neutralJointRotations.ys;
    return {ys.data(), ys.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getNeutralJointRotationZs() const {
    const auto& zs = dna.definition.neutralJointRotations.zs;
    return {zs.data(), zs.size()};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getGUIToRawInputIndices() const {
    const auto& inputIndices = dna.behavior.controls.conditionals.inputIndices;
    return {inputIndices.data(), inputIndices.size()};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getGUIToRawOutputIndices() const {
    const auto& outputIndices = dna.behavior.controls.conditionals.outputIndices;
    return {outputIndices.data(), outputIndices.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getGUIToRawFromValues() const {
    const auto& fromValues = dna.behavior.controls.conditionals.fromValues;
    return {fromValues.data(), fromValues.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getGUIToRawToValues() const {
    const auto& toValues = dna.behavior.controls.conditionals.toValues;
    return {toValues.data(), toValues.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getGUIToRawSlopeValues() const {
    const auto& slopeValues = dna.behavior.controls.conditionals.slopeValues;
    return {slopeValues.data(), slopeValues.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getGUIToRawCutValues() const {
    const auto& cutValues = dna.behavior.controls.conditionals.cutValues;
    return {cutValues.data(), cutValues.size()};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getPSDCount() const {
    return dna.behavior.controls.psdCount;
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getPSDRowIndices() const {
    const auto& rows = dna.behavior.controls.psds.rows;
    return {rows.data(), rows.size()};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getPSDColumnIndices() const {
    const auto& columns = dna.behavior.controls.psds.columns;
    return {columns.data(), columns.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getPSDValues() const {
    const auto& values = dna.behavior.controls.psds.values;
    return {values.data(), values.size()};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getJointRowCount() const {
    return dna.behavior.joints.rowCount;
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getJointColumnCount() const {
    return dna.behavior.joints.colCount;
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getJointVariableAttributeIndices(std::uint16_t lod) const {
    if (cache.jointVariableAttributeIndices.getLODCount() == static_cast<std::uint16_t>(0)) {
        cache.populateJointVariableAttributeIndices(this);
    }
    return cache.jointVariableAttributeIndices.getIndices(lod);
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getJointGroupCount() const {
    return static_cast<std::uint16_t>(dna.behavior.joints.jointGroups.size());
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getJointGroupLODs(std::uint16_t jointGroupIndex) const {
    if (jointGroupIndex < dna.behavior.joints.jointGroups.size()) {
        const auto& lods = dna.behavior.joints.jointGroups[jointGroupIndex].lods;
        return {lods.data(), lods.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getJointGroupInputIndices(std::uint16_t jointGroupIndex) const {
    if (jointGroupIndex < dna.behavior.joints.jointGroups.size()) {
        const auto& inputIndices = dna.behavior.joints.jointGroups[jointGroupIndex].inputIndices;
        return {inputIndices.data(), inputIndices.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getJointGroupOutputIndices(std::uint16_t jointGroupIndex) const {
    if (jointGroupIndex < dna.behavior.joints.jointGroups.size()) {
        const auto& outputIndices = dna.behavior.joints.jointGroups[jointGroupIndex].outputIndices;
        return {outputIndices.data(), outputIndices.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getJointGroupValues(std::uint16_t jointGroupIndex) const {
    if (jointGroupIndex < dna.behavior.joints.jointGroups.size()) {
        const auto& values = dna.behavior.joints.jointGroups[jointGroupIndex].values;
        return {values.data(), values.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getJointGroupJointIndices(std::uint16_t jointGroupIndex) const {
    if (jointGroupIndex < dna.behavior.joints.jointGroups.size()) {
        const auto& jointIndices = dna.behavior.joints.jointGroups[jointGroupIndex].jointIndices;
        return {jointIndices.data(), jointIndices.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getBlendShapeChannelLODs() const {
    const auto& lods = dna.behavior.blendShapeChannels.lods;
    return {lods.data(), lods.size()};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getBlendShapeChannelInputIndices() const {
    const auto& inputIndices = dna.behavior.blendShapeChannels.inputIndices;
    return {inputIndices.data(), inputIndices.size()};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getBlendShapeChannelOutputIndices() const {
    const auto& outputIndices = dna.behavior.blendShapeChannels.outputIndices;
    return {outputIndices.data(), outputIndices.size()};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getAnimatedMapLODs() const {
    const auto& lods = dna.behavior.animatedMaps.lods;
    return {lods.data(), lods.size()};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getAnimatedMapInputIndices() const {
    const auto& inputIndices = dna.behavior.animatedMaps.conditionals.inputIndices;
    return {inputIndices.data(), inputIndices.size()};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getAnimatedMapOutputIndices() const {
    const auto& outputIndices = dna.behavior.animatedMaps.conditionals.outputIndices;
    return {outputIndices.data(), outputIndices.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getAnimatedMapFromValues() const {
    const auto& fromValues = dna.behavior.animatedMaps.conditionals.fromValues;
    return {fromValues.data(), fromValues.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getAnimatedMapToValues() const {
    const auto& toValues = dna.behavior.animatedMaps.conditionals.toValues;
    return {toValues.data(), toValues.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getAnimatedMapSlopeValues() const {
    const auto& slopeValues = dna.behavior.animatedMaps.conditionals.slopeValues;
    return {slopeValues.data(), slopeValues.size()};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getAnimatedMapCutValues() const {
    const auto& cutValues = dna.behavior.animatedMaps.conditionals.cutValues;
    return {cutValues.data(), cutValues.size()};
}

template<class TReaderBase>
inline std::uint32_t ReaderImpl<TReaderBase>::getVertexPositionCount(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        return static_cast<std::uint32_t>(dna.geometry.meshes[meshIndex].positions.xs.size());
    }
    return 0u;
}

template<class TReaderBase>
inline Position ReaderImpl<TReaderBase>::getVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& positions = dna.geometry.meshes[meshIndex].positions;
        if (vertexIndex < positions.size()) {
            return {positions.xs[vertexIndex], positions.ys[vertexIndex], positions.zs[vertexIndex]};
        }
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getVertexPositionXs(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& xPositions = dna.geometry.meshes[meshIndex].positions.xs;
        return {xPositions.data(), xPositions.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getVertexPositionYs(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& yPositions = dna.geometry.meshes[meshIndex].positions.ys;
        return {yPositions.data(), yPositions.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getVertexPositionZs(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& zPositions = dna.geometry.meshes[meshIndex].positions.zs;
        return {zPositions.data(), zPositions.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint32_t ReaderImpl<TReaderBase>::getVertexTextureCoordinateCount(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        return static_cast<std::uint32_t>(dna.geometry.meshes[meshIndex].textureCoordinates.us.size());
    }
    return 0u;
}

template<class TReaderBase>
inline TextureCoordinate ReaderImpl<TReaderBase>::getVertexTextureCoordinate(std::uint16_t meshIndex,
                                                                             std::uint32_t textureCoordinateIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& textureCoordinates = dna.geometry.meshes[meshIndex].textureCoordinates;
        if (textureCoordinateIndex < textureCoordinates.size()) {
            return {textureCoordinates.us[textureCoordinateIndex], textureCoordinates.vs[textureCoordinateIndex]};
        }
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getVertexTextureCoordinateUs(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        return dna.geometry.meshes[meshIndex].textureCoordinates.us;
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getVertexTextureCoordinateVs(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        return dna.geometry.meshes[meshIndex].textureCoordinates.vs;
    }
    return {};
}

template<class TReaderBase>
inline std::uint32_t ReaderImpl<TReaderBase>::getVertexNormalCount(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        return static_cast<std::uint32_t>(dna.geometry.meshes[meshIndex].normals.xs.size());
    }
    return 0u;
}

template<class TReaderBase>
inline Normal ReaderImpl<TReaderBase>::getVertexNormal(std::uint16_t meshIndex, std::uint32_t normalIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& normals = dna.geometry.meshes[meshIndex].normals;
        if (normalIndex < normals.size()) {
            return {normals.xs[normalIndex], normals.ys[normalIndex], normals.zs[normalIndex]};
        }
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getVertexNormalXs(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& xNormals = dna.geometry.meshes[meshIndex].normals.xs;
        return {xNormals.data(), xNormals.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getVertexNormalYs(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& yNormals = dna.geometry.meshes[meshIndex].normals.ys;
        return {yNormals.data(), yNormals.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getVertexNormalZs(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& zNormals = dna.geometry.meshes[meshIndex].normals.zs;
        return {zNormals.data(), zNormals.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint32_t ReaderImpl<TReaderBase>::getFaceCount(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        return static_cast<std::uint32_t>(dna.geometry.meshes[meshIndex].faces.size());
    }
    return 0u;
}

template<class TReaderBase>
inline ConstArrayView<std::uint32_t> ReaderImpl<TReaderBase>::getFaceVertexLayoutIndices(std::uint16_t meshIndex,
                                                                                         std::uint32_t faceIndex) const {
    const auto& meshes = dna.geometry.meshes;
    if ((meshIndex < meshes.size()) && (faceIndex < meshes[meshIndex].faces.size())) {
        const auto& layoutIndices = meshes[meshIndex].faces[faceIndex].layoutIndices;
        return {layoutIndices.data(), layoutIndices.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint32_t ReaderImpl<TReaderBase>::getVertexLayoutCount(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        return static_cast<std::uint32_t>(dna.geometry.meshes[meshIndex].layouts.positions.size());
    }
    return 0u;
}

template<class TReaderBase>
inline VertexLayout ReaderImpl<TReaderBase>::getVertexLayout(std::uint16_t meshIndex, std::uint32_t layoutIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& layouts = dna.geometry.meshes[meshIndex].layouts;
        if (layoutIndex < layouts.size()) {
            return {layouts.positions[layoutIndex], layouts.textureCoordinates[layoutIndex], layouts.normals[layoutIndex]};
        }
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint32_t> ReaderImpl<TReaderBase>::getVertexLayoutPositionIndices(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& positions = dna.geometry.meshes[meshIndex].layouts.positions;
        return {positions.data(), positions.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint32_t>
ReaderImpl<TReaderBase>::getVertexLayoutTextureCoordinateIndices(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& textureCoordinated = dna.geometry.meshes[meshIndex].layouts.textureCoordinates;
        return {textureCoordinated.data(), textureCoordinated.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint32_t> ReaderImpl<TReaderBase>::getVertexLayoutNormalIndices(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        const auto& normals = dna.geometry.meshes[meshIndex].layouts.normals;
        return {normals.data(), normals.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getMaximumInfluencePerVertex(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        return dna.geometry.meshes[meshIndex].maximumInfluencePerVertex;
    }
    return {};
}

template<class TReaderBase>
inline std::uint32_t ReaderImpl<TReaderBase>::getSkinWeightsCount(std::uint16_t meshIndex) const {
    const auto& meshes = dna.geometry.meshes;
    if (meshIndex < meshes.size()) {
        return static_cast<std::uint32_t>(meshes[meshIndex].skinWeights.size());
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getSkinWeightsValues(std::uint16_t meshIndex,
                                                                           std::uint32_t vertexIndex) const {
    const auto& meshes = dna.geometry.meshes;
    if ((meshIndex < meshes.size()) && (vertexIndex < meshes[meshIndex].skinWeights.size())) {
        const auto& weights = meshes[meshIndex].skinWeights[vertexIndex].weights;
        return {weights.data(), weights.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getSkinWeightsJointIndices(std::uint16_t meshIndex,
                                                                                         std::uint32_t vertexIndex) const {
    const auto& meshes = dna.geometry.meshes;
    if ((meshIndex < meshes.size()) && (vertexIndex < meshes[meshIndex].skinWeights.size())) {
        const auto& jointIndices = meshes[meshIndex].skinWeights[vertexIndex].jointIndices;
        return {jointIndices.data(), jointIndices.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getBlendShapeTargetCount(std::uint16_t meshIndex) const {
    if (meshIndex < dna.geometry.meshes.size()) {
        return static_cast<std::uint16_t>(dna.geometry.meshes[meshIndex].blendShapeTargets.size());
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getBlendShapeChannelIndex(std::uint16_t meshIndex,
                                                                        std::uint16_t blendShapeTargetIndex) const {
    const auto& meshes = dna.geometry.meshes;
    if ((meshIndex < meshes.size()) && (blendShapeTargetIndex < meshes[meshIndex].blendShapeTargets.size())) {
        return meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].blendShapeChannelIndex;
    }
    return {};
}

template<class TReaderBase>
inline std::uint32_t ReaderImpl<TReaderBase>::getBlendShapeTargetDeltaCount(std::uint16_t meshIndex,
                                                                            std::uint16_t blendShapeTargetIndex) const {
    const auto& meshes = dna.geometry.meshes;
    if ((meshIndex < meshes.size()) && (blendShapeTargetIndex < meshes[meshIndex].blendShapeTargets.size())) {
        return static_cast<std::uint32_t>(meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].deltas.xs.size());
    }
    return {};
}

template<class TReaderBase>
inline Delta ReaderImpl<TReaderBase>::getBlendShapeTargetDelta(std::uint16_t meshIndex,
                                                               std::uint16_t blendShapeTargetIndex,
                                                               std::uint32_t deltaIndex) const {
    const auto& meshes = dna.geometry.meshes;
    if ((meshIndex < meshes.size()) && (blendShapeTargetIndex < meshes[meshIndex].blendShapeTargets.size()) &&
        (deltaIndex < meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].deltas.size())) {
        const auto& deltas = meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].deltas;
        return {deltas.xs[deltaIndex], deltas.ys[deltaIndex], deltas.zs[deltaIndex]};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getBlendShapeTargetDeltaXs(std::uint16_t meshIndex,
                                                                                 std::uint16_t blendShapeTargetIndex) const {
    const auto& meshes = dna.geometry.meshes;
    if ((meshIndex < meshes.size()) && (blendShapeTargetIndex < meshes[meshIndex].blendShapeTargets.size())) {
        const auto& xDeltas = meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].deltas.xs;
        return {xDeltas.data(), xDeltas.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getBlendShapeTargetDeltaYs(std::uint16_t meshIndex,
                                                                                 std::uint16_t blendShapeTargetIndex) const {
    const auto& meshes = dna.geometry.meshes;
    if ((meshIndex < meshes.size()) && (blendShapeTargetIndex < meshes[meshIndex].blendShapeTargets.size())) {
        const auto& yDeltas = meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].deltas.ys;
        return {yDeltas.data(), yDeltas.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getBlendShapeTargetDeltaZs(std::uint16_t meshIndex,
                                                                                 std::uint16_t blendShapeTargetIndex) const {
    const auto& meshes = dna.geometry.meshes;
    if ((meshIndex < meshes.size()) && (blendShapeTargetIndex < meshes[meshIndex].blendShapeTargets.size())) {
        const auto& zDeltas = meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].deltas.zs;
        return {zDeltas.data(), zDeltas.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint32_t>
ReaderImpl<TReaderBase>::getBlendShapeTargetVertexIndices(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const {
    const auto& meshes = dna.geometry.meshes;
    if ((meshIndex < meshes.size()) && (blendShapeTargetIndex < meshes[meshIndex].blendShapeTargets.size())) {
        const auto& vertexIndices = meshes[meshIndex].blendShapeTargets[blendShapeTargetIndex].vertexIndices;
        return {vertexIndices.data(), vertexIndices.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getMLControlCount() const {
    return static_cast<std::uint16_t>(dna.machineLearnedBehavior.mlControlNames.size());
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getMLControlName(std::uint16_t index) const {
    if (index < dna.machineLearnedBehavior.mlControlNames.size()) {
        const auto& mlControlName = dna.machineLearnedBehavior.mlControlNames[index];
        return {mlControlName.data(), mlControlName.size()};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getNeuralNetworkCount() const {
    return static_cast<std::uint16_t>(dna.machineLearnedBehavior.neuralNetworks.size());
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getNeuralNetworkIndexListCount() const {
    return dna.machineLearnedBehavior.lodNeuralNetworkMapping.getIndexListCount();
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getNeuralNetworkIndicesForLOD(std::uint16_t lod) const {
    return dna.machineLearnedBehavior.lodNeuralNetworkMapping.getIndices(lod);
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getMeshRegionCount(std::uint16_t meshIndex) const {
    if (meshIndex < dna.machineLearnedBehavior.neuralNetworkToMeshRegion.regionNames.size()) {
        return static_cast<std::uint16_t>(dna.machineLearnedBehavior.neuralNetworkToMeshRegion.regionNames[meshIndex].size());
    }
    return {};
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getMeshRegionName(std::uint16_t meshIndex, std::uint16_t regionIndex) const {
    if ((meshIndex < dna.machineLearnedBehavior.neuralNetworkToMeshRegion.regionNames.size()) &&
        (regionIndex < dna.machineLearnedBehavior.neuralNetworkToMeshRegion.regionNames[meshIndex].size())) {
        return StringView{dna.machineLearnedBehavior.neuralNetworkToMeshRegion.regionNames[meshIndex][regionIndex]};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t>
ReaderImpl<TReaderBase>::getNeuralNetworkIndicesForMeshRegion(std::uint16_t meshIndex, std::uint16_t regionIndex) const {
    if (meshIndex < dna.machineLearnedBehavior.neuralNetworkToMeshRegion.indices.size()) {
        const auto& mesh = dna.machineLearnedBehavior.neuralNetworkToMeshRegion.indices[meshIndex];
        if (regionIndex < mesh.size()) {
            const auto& region = mesh[regionIndex];
            return {region.data(), region.size()};
        }
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getNeuralNetworkInputIndices(std::uint16_t netIndex) const {
    const auto& neuralNets = dna.machineLearnedBehavior.neuralNetworks;
    if (netIndex < neuralNets.size()) {
        return ConstArrayView<std::uint16_t>{neuralNets[netIndex].inputIndices};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getNeuralNetworkOutputIndices(std::uint16_t netIndex) const {
    const auto& neuralNets = dna.machineLearnedBehavior.neuralNetworks;
    if (netIndex < neuralNets.size()) {
        return ConstArrayView<std::uint16_t>{neuralNets[netIndex].outputIndices};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getNeuralNetworkLayerCount(std::uint16_t netIndex) const {
    const auto& neuralNets = dna.machineLearnedBehavior.neuralNetworks;
    if (netIndex < neuralNets.size()) {
        return static_cast<std::uint16_t>(neuralNets[netIndex].layers.size());
    }
    return {};
}

template<class TReaderBase>
inline ActivationFunction ReaderImpl<TReaderBase>::getNeuralNetworkLayerActivationFunction(std::uint16_t netIndex,
                                                                                           std::uint16_t layerIndex) const {
    const auto& neuralNets = dna.machineLearnedBehavior.neuralNetworks;
    if ((netIndex < neuralNets.size()) && (layerIndex < neuralNets[netIndex].layers.size())) {
        return static_cast<ActivationFunction>(neuralNets[netIndex].layers[layerIndex].activationFunction.functionId);
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float>
ReaderImpl<TReaderBase>::getNeuralNetworkLayerActivationFunctionParameters(std::uint16_t netIndex,
                                                                           std::uint16_t layerIndex) const {
    const auto& neuralNets = dna.machineLearnedBehavior.neuralNetworks;
    if ((netIndex < neuralNets.size()) && (layerIndex < neuralNets[netIndex].layers.size())) {
        return ConstArrayView<float>{neuralNets[netIndex].layers[layerIndex].activationFunction.parameters};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getNeuralNetworkLayerBiases(std::uint16_t netIndex,
                                                                                  std::uint16_t layerIndex) const {
    const auto& neuralNets = dna.machineLearnedBehavior.neuralNetworks;
    if ((netIndex < neuralNets.size()) && (layerIndex < neuralNets[netIndex].layers.size())) {
        return ConstArrayView<float>{neuralNets[netIndex].layers[layerIndex].biases};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getNeuralNetworkLayerWeights(std::uint16_t netIndex,
                                                                                   std::uint16_t layerIndex) const {
    const auto& neuralNets = dna.machineLearnedBehavior.neuralNetworks;
    if ((netIndex < neuralNets.size()) && (layerIndex < neuralNets[netIndex].layers.size())) {
        return ConstArrayView<float>{neuralNets[netIndex].layers[layerIndex].weights};
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getRBFPoseCount() const {
    return static_cast<std::uint16_t>(dna.rbfBehavior.poses.size());
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getRBFPoseName(std::uint16_t poseIndex) const {
    const auto& poses = dna.rbfBehavior.poses;
    if (poseIndex < poses.size()) {
        return poses[poseIndex].name;
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getRBFPoseJointOutputIndices(std::uint16_t poseIndex) const {
    const auto& poses = dna.rbfBehavior.poses;
    if (poseIndex < poses.size()) {
        if (cache.rbfPoseJointOutputIndices.size() == 0) {
            cache.populateRBFPoseJointOutputIndices(this);
        }
        return cache.rbfPoseJointOutputIndices[poseIndex];
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t>
ReaderImpl<TReaderBase>::getRBFPoseBlendShapeChannelOutputIndices(std::uint16_t poseIndex) const {
    const auto& poses = dna.rbfBehavior.poses;
    if (poseIndex < poses.size()) {
        if (cache.rbfBlendShapeChannelOutputIndices.size() == 0) {
            cache.populateRBFBlendShapeOutputIndices(this);
        }
        return cache.rbfBlendShapeChannelOutputIndices[poseIndex];
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getRBFPoseAnimatedMapOutputIndices(std::uint16_t poseIndex) const {
    const auto& poses = dna.rbfBehavior.poses;
    if (poseIndex < poses.size()) {
        if (cache.rbfAnimatedMapOutputIndices.size() == 0) {
            cache.populateRBFAnimatedMapOutputIndices(this);
        }
        return cache.rbfAnimatedMapOutputIndices[poseIndex];
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getRBFPoseJointOutputValues(std::uint16_t poseIndex) const {
    const auto& poses = dna.rbfBehavior.poses;
    if (poseIndex < poses.size()) {
        if (cache.rbfPoseJointOutputValues.size() == 0) {
            cache.populateRBFPoseJointOutputValues(this);
        }
        return cache.rbfPoseJointOutputValues[poseIndex];
    }
    return {};
}

template<class TReaderBase>
inline float ReaderImpl<TReaderBase>::getRBFPoseScale(std::uint16_t poseIndex) const {
    const auto& poses = dna.rbfBehavior.poses;
    if (poseIndex < poses.size()) {
        return poses[poseIndex].scale;
    }
    return 0.0f;
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getRBFPoseControlCount() const {
    return static_cast<std::uint16_t>(dna.rbfBehaviorExt.poseControlNames.size());
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getRBFPoseControlName(std::uint16_t poseControlIndex) const {
    if (poseControlIndex < dna.rbfBehaviorExt.poseControlNames.size()) {
        const auto& poseControlName = dna.rbfBehaviorExt.poseControlNames[poseControlIndex];
        return {poseControlName.data(), poseControlName.size()};
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getRBFPoseInputControlIndices(std::uint16_t poseIndex) const {
    const auto& poses = dna.rbfBehaviorExt.poses;
    if (poseIndex < poses.size()) {
        return poses[poseIndex].inputControlIndices;
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getRBFPoseOutputControlIndices(std::uint16_t poseIndex) const {
    const auto& poses = dna.rbfBehaviorExt.poses;
    if (poseIndex < poses.size()) {
        return poses[poseIndex].outputControlIndices;
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getRBFPoseOutputControlWeights(std::uint16_t poseIndex) const {
    const auto& poses = dna.rbfBehaviorExt.poses;
    if (poseIndex < poses.size()) {
        return poses[poseIndex].outputControlWeights;
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getRBFSolverCount() const {
    return static_cast<std::uint16_t>(dna.rbfBehavior.solvers.size());
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getRBFSolverIndexListCount() const {
    return dna.rbfBehavior.lodSolverMapping.getIndexListCount();
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getRBFSolverIndicesForLOD(std::uint16_t lod) const {
    return dna.rbfBehavior.lodSolverMapping.getIndices(lod);
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getRBFSolverName(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return dna.rbfBehavior.solvers[solverIndex].name;
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getRBFSolverRawControlIndices(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return dna.rbfBehavior.solvers[solverIndex].rawControlIndices;
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getRBFSolverPoseIndices(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return dna.rbfBehavior.solvers[solverIndex].poseIndices;
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getRBFSolverRawControlValues(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return dna.rbfBehavior.solvers[solverIndex].rawControlValues;
    }
    return {};
}

template<class TReaderBase>
inline RBFSolverType ReaderImpl<TReaderBase>::getRBFSolverType(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return static_cast<RBFSolverType>(dna.rbfBehavior.solvers[solverIndex].solverType);
    }
    return {};
}

template<class TReaderBase>
inline float ReaderImpl<TReaderBase>::getRBFSolverRadius(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return dna.rbfBehavior.solvers[solverIndex].radius;
    }
    return 0.0f;
}

template<class TReaderBase>
inline AutomaticRadius ReaderImpl<TReaderBase>::getRBFSolverAutomaticRadius(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return static_cast<AutomaticRadius>(dna.rbfBehavior.solvers[solverIndex].automaticRadius);
    }
    return {};
}

template<class TReaderBase>
inline float ReaderImpl<TReaderBase>::getRBFSolverWeightThreshold(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return dna.rbfBehavior.solvers[solverIndex].weightThreshold;
    }
    return 0.0f;
}

template<class TReaderBase>
inline RBFDistanceMethod ReaderImpl<TReaderBase>::getRBFSolverDistanceMethod(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return static_cast<RBFDistanceMethod>(dna.rbfBehavior.solvers[solverIndex].distanceMethod);
    }
    return {};
}

template<class TReaderBase>
inline RBFNormalizeMethod ReaderImpl<TReaderBase>::getRBFSolverNormalizeMethod(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return static_cast<RBFNormalizeMethod>(dna.rbfBehavior.solvers[solverIndex].normalizeMethod);
    }
    return {};
}

template<class TReaderBase>
inline RBFFunctionType ReaderImpl<TReaderBase>::getRBFSolverFunctionType(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return static_cast<RBFFunctionType>(dna.rbfBehavior.solvers[solverIndex].functionType);
    }
    return {};
}

template<class TReaderBase>
inline TwistAxis ReaderImpl<TReaderBase>::getRBFSolverTwistAxis(std::uint16_t solverIndex) const {
    if (dna.rbfBehavior.solvers.size() > solverIndex) {
        return static_cast<TwistAxis>(dna.rbfBehavior.solvers[solverIndex].twistAxis);
    }
    return {};
}

template<class TReaderBase>
inline TranslationRepresentation ReaderImpl<TReaderBase>::getJointTranslationRepresentation(std::uint16_t jointIndex) const {
    if (dna.jointBehaviorMetadata.jointRepresentations.size() > jointIndex) {
        return static_cast<TranslationRepresentation>(dna.jointBehaviorMetadata.jointRepresentations[jointIndex].translation);
    }
    return {};
}

template<class TReaderBase>
inline RotationRepresentation ReaderImpl<TReaderBase>::getJointRotationRepresentation(std::uint16_t jointIndex) const {
    if (dna.jointBehaviorMetadata.jointRepresentations.size() > jointIndex) {
        return static_cast<RotationRepresentation>(dna.jointBehaviorMetadata.jointRepresentations[jointIndex].rotation);
    }
    return {};
}

template<class TReaderBase>
inline ScaleRepresentation ReaderImpl<TReaderBase>::getJointScaleRepresentation(std::uint16_t jointIndex) const {
    if (dna.jointBehaviorMetadata.jointRepresentations.size() > jointIndex) {
        return static_cast<ScaleRepresentation>(dna.jointBehaviorMetadata.jointRepresentations[jointIndex].scale);
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getTwistCount() const {
    return static_cast<std::uint16_t>(dna.twistSwingBehavior.twists.size());
}

template<class TReaderBase>
inline TwistAxis ReaderImpl<TReaderBase>::getTwistSetupTwistAxis(std::uint16_t twistIndex) const {
    if (dna.twistSwingBehavior.twists.size() > twistIndex) {
        return static_cast<TwistAxis>(dna.twistSwingBehavior.twists[twistIndex].twistAxis);
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getTwistInputControlIndices(std::uint16_t twistIndex) const {
    if (dna.twistSwingBehavior.twists.size() > twistIndex) {
        return dna.twistSwingBehavior.twists[twistIndex].twistInputControlIndices;
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getTwistOutputJointIndices(std::uint16_t twistIndex) const {
    if (dna.twistSwingBehavior.twists.size() > twistIndex) {
        return dna.twistSwingBehavior.twists[twistIndex].twistOutputJointIndices;
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getTwistBlendWeights(std::uint16_t twistIndex) const {
    if (dna.twistSwingBehavior.twists.size() > twistIndex) {
        return dna.twistSwingBehavior.twists[twistIndex].twistBlendWeights;
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getSwingCount() const {
    return static_cast<std::uint16_t>(dna.twistSwingBehavior.swings.size());
}

template<class TReaderBase>
inline TwistAxis ReaderImpl<TReaderBase>::getSwingSetupTwistAxis(std::uint16_t swingIndex) const {
    if (dna.twistSwingBehavior.swings.size() > swingIndex) {
        return static_cast<TwistAxis>(dna.twistSwingBehavior.swings[swingIndex].twistAxis);
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getSwingInputControlIndices(std::uint16_t swingIndex) const {
    if (dna.twistSwingBehavior.swings.size() > swingIndex) {
        return dna.twistSwingBehavior.swings[swingIndex].swingInputControlIndices;
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getSwingOutputJointIndices(std::uint16_t swingIndex) const {
    if (dna.twistSwingBehavior.swings.size() > swingIndex) {
        return dna.twistSwingBehavior.swings[swingIndex].swingOutputJointIndices;
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getSwingBlendWeights(std::uint16_t swingIndex) const {
    if (dna.twistSwingBehavior.swings.size() > swingIndex) {
        return dna.twistSwingBehavior.swings[swingIndex].swingBlendWeights;
    }
    return {};
}

template<class TReaderBase>
inline bool ReaderImpl<TReaderBase>::isMLExtLoaded() const {
    return !dna.machineLearnedBehaviorExt.mlbTypeData.empty();
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getMLTypeCount() const {
    if (!isMLExtLoaded()) {
        return dna.machineLearnedBehavior.neuralNetworks.empty() ? 0u : 1u;
    }
    return static_cast<std::uint16_t>(dna.machineLearnedBehaviorExt.mlbTypeData.size());
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getMLOperationSetCount(std::uint16_t mlTypeIndex) const {
    if (!isMLExtLoaded()) {
        return dna.machineLearnedBehavior.neuralNetworks.empty() ? 0u : 3u;
    }
    if (dna.machineLearnedBehaviorExt.mlbTypeData.size() > mlTypeIndex) {
        return static_cast<std::uint16_t>(dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex].operations.size());
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getMLOperationCount(std::uint16_t mlTypeIndex,
                                                                  std::uint16_t mlOperationSetIndex) const {
    if (!isMLExtLoaded()) {
        return getNeuralNetworkCount();
    }
    if (mlTypeIndex < dna.machineLearnedBehaviorExt.mlbTypeData.size()) {
        const auto& mlbType = dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex];
        if (mlOperationSetIndex < mlbType.operations.size()) {
            return static_cast<std::uint16_t>(mlbType.operations[mlOperationSetIndex].size());
        }
    }
    return {};
}

template<class TReaderBase>
inline MachineLearnedBehaviorOperationType ReaderImpl<TReaderBase>::getMLOperationType(std::uint16_t mlTypeIndex,
                                                                                       std::uint16_t mlOperationSetIndex,
                                                                                       std::uint16_t mlOperationIndex) const {
    if (!isMLExtLoaded()) {
        static const MachineLearnedBehaviorOperationType types[] = {MachineLearnedBehaviorOperationType::Gather,
                                                                    MachineLearnedBehaviorOperationType::MLP,
                                                                    MachineLearnedBehaviorOperationType::Scatter};
        if (mlOperationSetIndex < getMLOperationSetCount(mlTypeIndex)) {
            return types[mlOperationSetIndex];
        }
        return {};
    }
    if (mlTypeIndex < dna.machineLearnedBehaviorExt.mlbTypeData.size()) {
        const auto& mlbType = dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex];
        if (mlOperationSetIndex < mlbType.operations.size()) {
            const auto& mlOperations = mlbType.operations[mlOperationSetIndex];
            if (mlOperationIndex < mlOperations.size()) {
                return static_cast<MachineLearnedBehaviorOperationType>(mlOperations[mlOperationIndex].operationType);
            }
        }
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint32_t> ReaderImpl<TReaderBase>::getMLOperationParameters(std::uint16_t mlTypeIndex,
                                                                                       std::uint16_t mlOperationSetIndex,
                                                                                       std::uint16_t mlOperationIndex) const {
    if (!isMLExtLoaded()) {
        if (cache.mlOperationParameters.empty()) {
            cache.populateMLOperationParameters(this);
        }
        if (mlOperationSetIndex < cache.mlOperationParameters.size()) {
            if (mlOperationIndex < cache.mlOperationParameters[mlOperationSetIndex].size()) {
                return cache.mlOperationParameters[mlOperationSetIndex][mlOperationIndex];
            }
        }
        return {};
    }
    if (mlTypeIndex < dna.machineLearnedBehaviorExt.mlbTypeData.size()) {
        const auto& mlbType = dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex];
        if (mlOperationSetIndex < mlbType.operations.size()) {
            const auto& mlOperations = mlbType.operations[mlOperationSetIndex];
            if (mlOperationIndex < mlOperations.size()) {
                return mlOperations[mlOperationIndex].parameters;
            }
        }
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t>
ReaderImpl<TReaderBase>::getMLOperationDependencyOperationSetIndices(std::uint16_t mlTypeIndex,
                                                                     std::uint16_t mlOperationSetIndex,
                                                                     std::uint16_t mlOperationIndex) const {
    if (!isMLExtLoaded()) {
        if (cache.mlDependencyOperationSetIndices.empty()) {
            cache.populateMLDependencyOperationSetIndices(this);
        }
        if (mlOperationSetIndex < cache.mlDependencyOperationSetIndices.size()) {
            if (mlOperationIndex < cache.mlDependencyOperationSetIndices[mlOperationSetIndex].size()) {
                return cache.mlDependencyOperationSetIndices[mlOperationSetIndex][mlOperationIndex];
            }
        }
        return {};
    }
    if (mlTypeIndex < dna.machineLearnedBehaviorExt.mlbTypeData.size()) {
        const auto& mlbType = dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex];
        if (mlOperationSetIndex < mlbType.operations.size()) {
            const auto& mlOperations = mlbType.operations[mlOperationSetIndex];
            if (mlOperationIndex < mlOperations.size()) {
                return mlOperations[mlOperationIndex].mlOperationSetDependencyIndices;
            }
        }
    }
    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t>
ReaderImpl<TReaderBase>::getMLOperationDependencyOperationIndices(std::uint16_t mlTypeIndex,
                                                                  std::uint16_t mlOperationSetIndex,
                                                                  std::uint16_t mlOperationIndex) const {
    if (!isMLExtLoaded()) {
        if (cache.mlDependencyOperationIndices.empty()) {
            cache.populateMLDependencyOperationIndices(this);
        }
        if (mlOperationSetIndex < cache.mlDependencyOperationIndices.size()) {
            if (mlOperationIndex < cache.mlDependencyOperationIndices[mlOperationSetIndex].size()) {
                return cache.mlDependencyOperationIndices[mlOperationSetIndex][mlOperationIndex];
            }
        }
        return {};
    }
    if (mlTypeIndex < dna.machineLearnedBehaviorExt.mlbTypeData.size()) {
        const auto& mlbType = dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex];
        if (mlOperationSetIndex < mlbType.operations.size()) {
            const auto& mlOperations = mlbType.operations[mlOperationSetIndex];
            if (mlOperationIndex < mlOperations.size()) {
                return mlOperations[mlOperationIndex].mlOperationDependencyIndices;
            }
        }
    }
    return {};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getMLOperationIndexListCount(std::uint16_t mlTypeIndex,
                                                                           std::uint16_t mlOperationSetIndex) const {
    if (!isMLExtLoaded()) {
        return getNeuralNetworkIndexListCount();
    }

    if (mlTypeIndex < dna.machineLearnedBehaviorExt.mlbTypeData.size()) {
        const auto& mlbType = dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex];
        if (mlOperationSetIndex < mlbType.lodMLOperationMappings.size()) {
            return mlbType.lodMLOperationMappings[mlOperationSetIndex].getIndexListCount();
        }
    }

    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getMLOperationIndicesForLOD(std::uint16_t mlTypeIndex,
                                                                                          std::uint16_t mlOperationSetIndex,
                                                                                          std::uint16_t lod) const {
    if (!isMLExtLoaded()) {
        return getNeuralNetworkIndicesForLOD(lod);
    }

    if (mlTypeIndex < dna.machineLearnedBehaviorExt.mlbTypeData.size()) {
        const auto& mlbType = dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex];
        if (mlOperationSetIndex < mlbType.lodMLOperationMappings.size()) {
            return mlbType.lodMLOperationMappings[mlOperationSetIndex].getIndices(lod);
        }
    }

    return {};
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getMLJointsInputIndices() const {
    if (!isMLExtLoaded()) {
        return {};
    }
    return dna.machineLearnedBehaviorExt.mlbJoints.inputIndices;
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getMLJointsOutputIndices() const {
    if (!isMLExtLoaded()) {
        return {};
    }
    return dna.machineLearnedBehaviorExt.mlbJoints.outputIndices;
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getMLJointsParameterKeys() const {
    if (!isMLExtLoaded()) {
        return {};
    }
    return dna.machineLearnedBehaviorExt.mlbJoints.parameterKeys;
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getMLJointsParameterValues() const {
    if (!isMLExtLoaded()) {
        return {};
    }
    return dna.machineLearnedBehaviorExt.mlbJoints.parameterValues;
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

}  // namespace dnac
