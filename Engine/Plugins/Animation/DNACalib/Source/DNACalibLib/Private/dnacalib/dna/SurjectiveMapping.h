// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace dnac {

template<typename TFrom, typename TTo = TFrom>
struct SurjectiveMapping {
public:
    struct Pair {
        TFrom from;
        TTo to;
    };

public:
    explicit SurjectiveMapping(MemoryResource* memRes) :
        from{memRes},
        to{memRes} {
    }

    Pair get(std::size_t index) const {
        assert(index < size());
        return {from[index], to[index]};
    }

    void add(TFrom from_, TTo to_) {
        from.push_back(from_);
        to.push_back(to_);
    }

    void set(std::size_t index, TFrom from_, TTo to_) {
        if (index >= size()) {
            from.resize(index + 1ul);
            to.resize(index + 1ul);
        }
        from[index] = from_;
        to[index] = to_;
    }

    void removeIf(std::function<bool(const TFrom&, const TTo&)> predicate) {
        assert(from.size() == to.size());

        auto itFrom = from.begin();
        auto itTo = to.begin();

        while (itFrom != from.end()) {
            if (predicate(*itFrom, *itTo)) {
                itFrom = from.erase(itFrom);
                itTo = to.erase(itTo);
            } else {
                ++itFrom;
                ++itTo;
            }
        }
    }

    void updateFrom(const UnorderedMap<TFrom, TFrom>& mapping) {
        update(from, mapping);
    }

    void updateTo(const UnorderedMap<TTo, TTo>& mapping) {
        update(to, mapping);
    }

    std::size_t size() const {
        assert(from.size() == to.size());
        return from.size();
    }

    void clear() {
        from.clear();
        to.clear();
    }

private:
    template<typename U>
    void update(Vector<U>& target, const UnorderedMap<U, U>& mapping) {
        std::transform(target.begin(), target.end(), target.begin(), [&mapping](U oldValue) { return mapping.at(oldValue); });
    }

protected:
    Vector<TFrom> from;
    Vector<TTo> to;
};

}  // namespace dnac
