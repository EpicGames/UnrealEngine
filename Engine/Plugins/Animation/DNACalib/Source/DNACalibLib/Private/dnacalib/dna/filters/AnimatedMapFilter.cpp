// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/dna/filters/AnimatedMapFilter.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/filters/Remap.h"
#include "dnacalib/utils/Extd.h"

namespace dnac {

AnimatedMapFilter::AnimatedMapFilter(MemoryResource* memRes_) :
    memRes{memRes_},
    passingIndices{memRes},
    remappedIndices{memRes} {
}

void AnimatedMapFilter::configure(std::uint16_t animatedMapCount,
                                  UnorderedSet<std::uint16_t> allowedAnimatedMapIndices,
                                  Matrix<std::uint16_t> lodIndices) {
    passingIndices = std::move(allowedAnimatedMapIndices);
    // Fill the structure that maps indices prior to deletion to indices after deletion
    remappedIndices.clear();
    remap(animatedMapCount, passingIndices, remappedIndices);
    animatedMapLODIndices = std::move(lodIndices);
}

void AnimatedMapFilter::apply(RawDefinition& dest) {
    // Fix indices so they match the same elements as earlier (but their
    // actual position changed with the deletion of the unneeded entries)
    dest.lodAnimatedMapMapping.mapIndices([this](std::uint16_t value) { return remappedIndices.at(value); });
    // Delete elements that are not referenced by the new subset of LODs
    extd::filter(dest.animatedMapNames, extd::byPosition(passingIndices));
}

void AnimatedMapFilter::apply(RawBehavior& dest) {
    UnorderedSet<std::size_t> indicesToDelete{memRes};

    // Remove output indices of animated maps to remove and update LODs
    extd::filter(dest.animatedMaps.conditionals.outputIndices,
                 [this, &indicesToDelete, &dest](std::uint16_t outputIndex, std::size_t index) {
                     if (!passes(outputIndex)) {
                         indicesToDelete.insert(index);
                         for (std::uint16_t lodIndex = 0; lodIndex < static_cast<std::uint16_t>(animatedMapLODIndices.size());
                              ++lodIndex) {
                             const auto& lodIndices = animatedMapLODIndices[lodIndex];
                             if (extd::contains(lodIndices, outputIndex)) {
                                 dest.animatedMaps.lods[lodIndex]--;
                             }
                         }
                         return false;
                     }
                     return true;
                 });

    // Remap remaining output indices
    for (auto& outputIdx : dest.animatedMaps.conditionals.outputIndices) {
        outputIdx = remappedIndices[outputIdx];
    }

    // Remove input indices associated with the removed output indices
    extd::filter(dest.animatedMaps.conditionals.inputIndices, [&indicesToDelete](std::uint16_t /*unused*/, std::size_t index) {
        return (indicesToDelete.find(index) == indicesToDelete.end());
    });

    // Remove from values associated with the removed output indices
    extd::filter(dest.animatedMaps.conditionals.fromValues, [&indicesToDelete](float /*unused*/, std::size_t index) {
        return (indicesToDelete.find(index) == indicesToDelete.end());
    });

    // Remove to values associated with the removed output indices
    extd::filter(dest.animatedMaps.conditionals.toValues, [&indicesToDelete](float /*unused*/, std::size_t index) {
        return (indicesToDelete.find(index) == indicesToDelete.end());
    });

    // Remove slope values associated with the removed output indices
    extd::filter(dest.animatedMaps.conditionals.slopeValues, [&indicesToDelete](float /*unused*/, std::size_t index) {
        return (indicesToDelete.find(index) == indicesToDelete.end());
    });

    // Remove cut values associated with the removed output indices
    extd::filter(dest.animatedMaps.conditionals.cutValues, [&indicesToDelete](float /*unused*/, std::size_t index) {
        return (indicesToDelete.find(index) == indicesToDelete.end());
    });
}

bool AnimatedMapFilter::passes(std::uint16_t index) const {
    return extd::contains(passingIndices, index);
}

}  // namespace dnac
