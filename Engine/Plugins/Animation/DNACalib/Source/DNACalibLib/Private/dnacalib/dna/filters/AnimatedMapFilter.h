// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"

#include <cstdint>

namespace dnac {

struct RawBehavior;
struct RawDefinition;

class AnimatedMapFilter {
public:
    explicit AnimatedMapFilter(MemoryResource* memRes_);
    void configure(std::uint16_t animatedMapCount,
                   UnorderedSet<std::uint16_t> allowedAnimatedMapIndices,
                   Matrix<std::uint16_t> lodIndices);
    void apply(RawDefinition& dest);
    void apply(RawBehavior& dest);
    bool passes(std::uint16_t index) const;

private:
    MemoryResource* memRes;
    UnorderedSet<std::uint16_t> passingIndices;
    UnorderedMap<std::uint16_t, std::uint16_t> remappedIndices;
    Matrix<std::uint16_t> animatedMapLODIndices;
};

}  // namespace dnac
