// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/dna/filters/BlendShapeFilter.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/filters/Remap.h"
#include "dnacalib/utils/Extd.h"

namespace dnac {

BlendShapeFilter::BlendShapeFilter(MemoryResource* memRes_) :
    memRes{memRes_},
    passingIndices{memRes},
    remappedIndices{memRes},
    newBlendShapeLODs{memRes} {
}

void BlendShapeFilter::configure(std::uint16_t blendShapeCount,
                                 UnorderedSet<std::uint16_t> allowedBlendShapeIndices,
                                 Vector<std::uint16_t> blendShapeLODs) {
    passingIndices = std::move(allowedBlendShapeIndices);
    // Fill the structure that maps indices prior to deletion to indices after deletion
    remappedIndices.clear();
    remap(blendShapeCount, passingIndices, remappedIndices);
    newBlendShapeLODs = std::move(blendShapeLODs);
}

void BlendShapeFilter::apply(RawDefinition& dest) {
    // Fix indices so they match the same elements as earlier (but their
    // actual position changed with the deletion of the unneeded entries)
    dest.lodBlendShapeMapping.mapIndices([this](std::uint16_t value) { return remappedIndices.at(value); });
    // Delete elements that are not referenced by the new subset of LODs
    extd::filter(dest.blendShapeChannelNames, extd::byPosition(passingIndices));
    // Delete entries from other mappings that reference any of the deleted elements
    auto ignoredByLODConstraint = [this](std::uint16_t /*unused*/, std::uint16_t blendShapeIndex) {
        return !extd::contains(passingIndices, blendShapeIndex);
    };
    dest.meshBlendShapeChannelMapping.removeIf(ignoredByLODConstraint);
    dest.meshBlendShapeChannelMapping.updateTo(remappedIndices);
}

void BlendShapeFilter::apply(RawBehavior& dest) {
    UnorderedSet<std::size_t> indicesToDelete{memRes};

    // Remove output indices of blend shapes to remove
    extd::filter(dest.blendShapeChannels.outputIndices, [this, &indicesToDelete](std::uint16_t outputIndex, std::size_t index) {
        if (!passes(outputIndex)) {
            indicesToDelete.insert(index);
            return false;
        }
        return true;
    });

    // Remap remaining output indices
    for (auto& outputIdx : dest.blendShapeChannels.outputIndices) {
        outputIdx = remapped(outputIdx);
    }

    // Remove input indices associated with the removed output indices
    extd::filter(dest.blendShapeChannels.inputIndices, [&indicesToDelete](std::uint16_t /*unused*/, std::size_t index) {
        return (indicesToDelete.find(index) == indicesToDelete.end());
    });

    // Set new LODs
    assert(newBlendShapeLODs.size() == dest.blendShapeChannels.lods.size());
    dest.blendShapeChannels.lods.assign(newBlendShapeLODs.begin(), newBlendShapeLODs.end());
}

void BlendShapeFilter::apply(RawMesh& dest) {
    // Remove blend shape targets of blend shapes to remove
    extd::filter(dest.blendShapeTargets, [this](const RawBlendShapeTarget& bsTarget, std::size_t /*unused*/) {
        return passes(bsTarget.blendShapeChannelIndex);
    });

    // Remap blend shape targets
    for (auto& bsTarget : dest.blendShapeTargets) {
        bsTarget.blendShapeChannelIndex = remapped(bsTarget.blendShapeChannelIndex);
    }
}

bool BlendShapeFilter::passes(std::uint16_t index) const {
    return extd::contains(passingIndices, index);
}

std::uint16_t BlendShapeFilter::remapped(std::uint16_t index) const {
    return remappedIndices.at(index);
}

}  // namespace dnac
