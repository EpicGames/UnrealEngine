// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"

#include <cstdint>

namespace dnac {

struct RawBehavior;
struct RawDefinition;
struct RawMesh;

class BlendShapeFilter {
public:
    explicit BlendShapeFilter(MemoryResource* memRes_);
    void configure(std::uint16_t blendShapeCount,
                   UnorderedSet<std::uint16_t> allowedBlendShapeIndices,
                   Vector<std::uint16_t> blendShapeLODs);
    void apply(RawDefinition& dest);
    void apply(RawBehavior& dest);
    void apply(RawMesh& dest);
    bool passes(std::uint16_t index) const;
    std::uint16_t remapped(std::uint16_t index) const;

private:
    MemoryResource* memRes;
    UnorderedSet<std::uint16_t> passingIndices;
    UnorderedMap<std::uint16_t, std::uint16_t> remappedIndices;
    Vector<std::uint16_t> newBlendShapeLODs;
};

}  // namespace dnac
