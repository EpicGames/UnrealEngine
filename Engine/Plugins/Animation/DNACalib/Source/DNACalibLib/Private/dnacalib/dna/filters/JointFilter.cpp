// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/dna/filters/JointFilter.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/filters/Remap.h"
#include "dnacalib/utils/Extd.h"

namespace dnac {

JointFilter::JointFilter(MemoryResource* memRes_) :
    memRes{memRes_},
    passingIndices{memRes_},
    remappedIndices{memRes_},
    option{Option::All},
    rootJointIndex{} {
}

void JointFilter::configure(std::uint16_t jointCount, UnorderedSet<std::uint16_t> allowedJointIndices, Option option_) {
    option = option_;
    passingIndices = std::move(allowedJointIndices);
    // Fill the structure that maps indices prior to deletion to indices after deletion
    remappedIndices.clear();
    remap(jointCount, passingIndices, remappedIndices);
}

void JointFilter::apply(RawDefinition& dest) {
    if (option != Option::All) {
        return;
    }
    // Fix indices so they match the same elements as earlier (but their
    // actual position changed with the deletion of the unneeded entries)
    dest.lodJointMapping.mapIndices([this](std::uint16_t value) { return remappedIndices.at(value); });
    // Delete elements that are not referenced by the new subset of LODs
    extd::filter(dest.jointNames, extd::byPosition(passingIndices));
    extd::filter(dest.jointHierarchy, extd::byPosition(passingIndices));
    // Fix joint hierarchy indices
    for (auto& jntIdx : dest.jointHierarchy) {
        jntIdx = remappedIndices[jntIdx];
    }
    // Find root joint index
    for (std::uint16_t jointIdx = {}; jointIdx < dest.jointHierarchy.size(); ++jointIdx) {
        if (dest.jointHierarchy[jointIdx] == jointIdx) {
            rootJointIndex = jointIdx;
            break;
        }
    }
    // Delete entries from other mappings that reference any of the deleted elements
    extd::filter(dest.neutralJointTranslations.xs, extd::byPosition(passingIndices));
    extd::filter(dest.neutralJointTranslations.ys, extd::byPosition(passingIndices));
    extd::filter(dest.neutralJointTranslations.zs, extd::byPosition(passingIndices));
    extd::filter(dest.neutralJointRotations.xs, extd::byPosition(passingIndices));
    extd::filter(dest.neutralJointRotations.ys, extd::byPosition(passingIndices));
    extd::filter(dest.neutralJointRotations.zs, extd::byPosition(passingIndices));
}

void JointFilter::apply(RawBehavior& dest) {
    static constexpr std::uint16_t jointAttributeCount = 9u;

    for (auto& jointGroup : dest.joints.jointGroups) {
        if (option == Option::All) {
            // Remove joint index from joint group and remap joint indices
            extd::filter(jointGroup.jointIndices,
                         [this](std::uint16_t jntIdx, std::size_t /*unused*/) { return passes(jntIdx); });
            for (auto& jntIdx : jointGroup.jointIndices) {
                jntIdx = remapped(jntIdx);
            }
        }
        // Collect row indices of removed output indices to be used for joint delta removal
        UnorderedSet<std::size_t> rowsToDelete(memRes);
        // Remove output indices belonging to the deletable joint
        extd::filter(jointGroup.outputIndices, [this, &rowsToDelete](std::uint16_t outputIndex, std::size_t rowIndex) {
            const auto jointIndex = static_cast<std::uint16_t>(outputIndex / jointAttributeCount);
            if (!passes(jointIndex)) {
                rowsToDelete.insert(rowIndex);
                return false;
            }
            return true;
        });

        if (option == Option::All) {
            // Remap the rest of output indices
            for (auto& attrIdx : jointGroup.outputIndices) {
                const auto jntIdx = static_cast<std::uint16_t>(attrIdx / jointAttributeCount);
                const auto relAttrIdx = attrIdx - (jntIdx * jointAttributeCount);
                attrIdx = static_cast<std::uint16_t>(remapped(jntIdx) * jointAttributeCount + relAttrIdx);
            }
        }

        // If no animation data remains, there's no point in keeping input indices
        const auto jointGroupColumnCount = static_cast<std::uint16_t>(jointGroup.inputIndices.size());
        if (jointGroup.outputIndices.empty()) {
            jointGroup.inputIndices.clear();
        }

        // Remove joint deltas associated with the removed output indices
        extd::filter(jointGroup.values, [&rowsToDelete, jointGroupColumnCount](float /*unused*/, std::size_t index) {
            const std::uint16_t rowIndex = static_cast<std::uint16_t>(index / jointGroupColumnCount);
            return (rowsToDelete.find(rowIndex) == rowsToDelete.end());
        });
        // Recompute LODs
        for (auto& lod : jointGroup.lods) {
            std::uint16_t decrementBy = {};
            for (const auto rowIndex : rowsToDelete) {
                if (rowIndex < lod) {
                    ++decrementBy;
                }
            }
            lod = static_cast<std::uint16_t>(lod - decrementBy);
        }
    }
}

void JointFilter::apply(RawVertexSkinWeights& dest) {
    if (option != Option::All) {
        return;
    }

    auto itWeightSrc = dest.weights.begin();
    auto itWeightDst = itWeightSrc;
    auto itJointSrc = dest.jointIndices.begin();
    auto itJointDst = itJointSrc;
    float discardedWeights = 0.0f;
    while (itJointSrc != dest.jointIndices.end()) {
        if (passes(*itJointSrc)) {
            *itJointDst = *itJointSrc;
            ++itJointSrc;
            ++itJointDst;

            *itWeightDst = *itWeightSrc;
            ++itWeightSrc;
            ++itWeightDst;
        } else {
            discardedWeights += *itWeightSrc;
            ++itJointSrc;
            ++itWeightSrc;
        }
    }
    dest.jointIndices.resize(static_cast<std::size_t>(std::distance(dest.jointIndices.begin(), itJointDst)));
    dest.weights.resize(static_cast<std::size_t>(std::distance(dest.weights.begin(), itWeightDst)));
    assert(dest.jointIndices.size() == dest.weights.size());

    if (passingIndices.empty()) {
        return;
    }

    if (dest.jointIndices.empty()) {
        // Reassign complete influence to root joint
        dest.jointIndices.resize_uninitialized(1ul);
        dest.jointIndices[0ul] = rootJointIndex;
        dest.weights.resize_uninitialized(1ul);
        dest.weights[0ul] = 1.0f;
    } else {
        // Normalize weights
        for (auto& jntIdx : dest.jointIndices) {
            jntIdx = remapped(jntIdx);
        }

        const float normalizationRatio = 1.0f / (1.0f - discardedWeights);
        for (auto& weight : dest.weights) {
            weight *= normalizationRatio;
        }
    }
}

void JointFilter::apply(RawJointBehaviorMetadata& dest) {
    const auto jointCount = static_cast<std::uint16_t>(dest.jointRepresentations.size());
    std::uint16_t newCount = {};
    for (std::uint16_t ji = {}; ji < jointCount; ++ji) {
        if (passes(ji)) {
            dest.jointRepresentations[remapped(ji)] = dest.jointRepresentations[ji];
            newCount++;
        }
    }
    dest.jointRepresentations.resize(newCount);
}

void JointFilter::apply(RawTwistSwingBehavior& dest) {
    if (option != Option::All) {
        return;
    }

    auto filterTwists = [this](RawTwistSwingBehavior& tsbh) {
        auto& twists = tsbh.twists;
        const auto twistCount = static_cast<std::uint16_t>(twists.size());
        std::uint16_t di = {};
        for (std::uint16_t ti = {}; ti < twistCount; ++ti) {
            const auto& src = twists[ti];
            auto& dst = twists[di];

            dst.twistAxis = src.twistAxis;
            dst.twistInputControlIndices = src.twistInputControlIndices;

            const auto outputIndicesCount = static_cast<std::uint16_t>(src.twistOutputJointIndices.size());
            dst.twistOutputJointIndices.resize(outputIndicesCount);
            dst.twistBlendWeights.resize(outputIndicesCount);
            std::uint16_t doi = {};
            for (std::uint16_t soi = {}; soi < outputIndicesCount; ++soi) {
                const auto srcJointIndex = src.twistOutputJointIndices[soi];
                if (passes(srcJointIndex)) {
                    dst.twistOutputJointIndices[doi] = remapped(srcJointIndex);
                    dst.twistBlendWeights[doi] = src.twistBlendWeights[soi];
                    doi++;
                }
            }

            if (doi != 0u) {
                dst.twistBlendWeights.resize(doi);
                dst.twistOutputJointIndices.resize(doi);
                di++;
            }
        }
        twists.resize(di, RawTwist{memRes});
    };

    auto filterSwings = [this](RawTwistSwingBehavior& tsbh) {
        auto& swings = tsbh.swings;
        const auto swingCount = static_cast<std::uint16_t>(swings.size());
        std::uint16_t di = {};
        for (std::uint16_t si = {}; si < swingCount; ++si) {
            const auto& src = swings[si];
            auto& dst = swings[di];

            dst.twistAxis = src.twistAxis;
            dst.swingInputControlIndices = src.swingInputControlIndices;

            const auto outputIndicesCount = static_cast<std::uint16_t>(src.swingOutputJointIndices.size());
            dst.swingOutputJointIndices.resize(outputIndicesCount);
            dst.swingBlendWeights.resize(outputIndicesCount);
            std::uint16_t doi = {};
            for (std::uint16_t soi = {}; soi < outputIndicesCount; ++soi) {
                const auto srcJointIndex = src.swingOutputJointIndices[soi];
                if (passes(srcJointIndex)) {
                    dst.swingOutputJointIndices[doi] = remapped(srcJointIndex);
                    dst.swingBlendWeights[doi] = src.swingBlendWeights[soi];
                    doi++;
                }
            }

            if (doi != 0u) {
                dst.swingBlendWeights.resize(doi);
                dst.swingOutputJointIndices.resize(doi);
                di++;
            }
        }
        swings.resize(di, RawSwing{memRes});
    };

    filterTwists(dest);
    filterSwings(dest);
}

bool JointFilter::passes(std::uint16_t index) const {
    return extd::contains(passingIndices, index);
}

std::uint16_t JointFilter::remapped(std::uint16_t oldIndex) const {
    return remappedIndices.at(oldIndex);
}

std::uint16_t JointFilter::filteredJointCount() const {
    return static_cast<std::uint16_t>(passingIndices.size());
}

}  // namespace dnac
