// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/dna/filters/MeshFilter.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"
#include "dnacalib/dna/filters/Remap.h"
#include "dnacalib/utils/Extd.h"

namespace dnac {

MeshFilter::MeshFilter(MemoryResource* memRes_) :
    memRes{memRes_},
    passingIndices{memRes},
    remappedIndices{memRes} {
}

void MeshFilter::configure(std::uint16_t meshCount, UnorderedSet<std::uint16_t> allowedMeshIndices) {
    passingIndices = std::move(allowedMeshIndices);
    // Fill the structure that maps indices prior to deletion to indices after deletion
    remappedIndices.clear();
    remap(meshCount, passingIndices, remappedIndices);
}

void MeshFilter::apply(RawDefinition& dest) {
    // Fix indices so they match the same elements as earlier (but their
    // actual position changed with the deletion of the unneeded entries)
    dest.lodMeshMapping.mapIndices([this](std::uint16_t value) { return remappedIndices.at(value); });
    // Delete elements that are not referenced by the new subset of LODs
    extd::filter(dest.meshNames, extd::byPosition(passingIndices));
    // Delete entries from other mappings that reference any of the deleted elements
    auto ignoredByLODConstraint = [this](std::uint16_t meshIndex, std::uint16_t /*unused*/) {
        return !extd::contains(passingIndices, meshIndex);
    };
    dest.meshBlendShapeChannelMapping.removeIf(ignoredByLODConstraint);
    dest.meshBlendShapeChannelMapping.updateFrom(remappedIndices);
}

void MeshFilter::apply(RawMachineLearnedBehavior& dest) {
    // Delete region names belonging to meshes that are not referenced by the new subset of LODs
    extd::filter(dest.neuralNetworkToMeshRegion.regionNames, extd::byPosition(passingIndices));
    extd::filter(dest.neuralNetworkToMeshRegion.indices, extd::byPosition(passingIndices));
}

bool MeshFilter::passes(std::uint16_t index) const {
    return extd::contains(passingIndices, index);
}

}  // namespace dnac
