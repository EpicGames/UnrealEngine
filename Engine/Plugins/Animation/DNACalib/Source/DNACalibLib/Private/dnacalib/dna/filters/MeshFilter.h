// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"

#include <cstdint>

namespace dnac {

struct RawDefinition;
struct RawMachineLearnedBehavior;

class MeshFilter {
public:
    explicit MeshFilter(MemoryResource* memRes_);
    void configure(std::uint16_t meshCount, UnorderedSet<std::uint16_t> allowedMeshIndices);
    void apply(RawDefinition& dest);
    void apply(RawMachineLearnedBehavior& dest);
    bool passes(std::uint16_t index) const;

private:
    MemoryResource* memRes;
    UnorderedSet<std::uint16_t> passingIndices;
    UnorderedMap<std::uint16_t, std::uint16_t> remappedIndices;
};

}  // namespace dnac
