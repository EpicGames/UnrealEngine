// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"
#include "dnacalib/types/Aliases.h"

#include <iterator>
#include <limits>

namespace dnac {

class BoundingBox {
public:
    static constexpr float defaultAlpha = 0.0003f;

private:
    template<typename TIter>
    using unqualified_iter_value_type = typename std::remove_cv<typename std::iterator_traits<TIter>::value_type>::type;

public:
    template<typename TIter,
             typename std::enable_if<std::is_same<unqualified_iter_value_type<TIter>, fvec2>::value, void>::type* = nullptr>
    BoundingBox(TIter begin, TIter end, float alpha) :
        min{std::numeric_limits<float>::max(), std::numeric_limits<float>::max()},
        max{std::numeric_limits<float>::min(), std::numeric_limits<float>::min()} {

        for (auto it = begin; it < end; it++) {
            const auto& vertex = *it;
            min[0] = std::min(vertex[0], min[0]);
            min[1] = std::min(vertex[1], min[1]);
            max[0] = std::max(vertex[0], max[0]);
            max[1] = std::max(vertex[1], max[1]);
        }
        min -= alpha;
        max += alpha;
    }

    template<typename TContainer>
    explicit BoundingBox(TContainer container, float alpha = defaultAlpha) :
        BoundingBox(std::begin(container), std::end(container), alpha) {
    }

    bool contains(fvec2 point) const {
        return point[0] >= min[0] && point[0] <= max[0] && point[1] >= min[1] && point[1] <= max[1];
    }

    float distance(fvec2 point) const {
        const float dx = std::max({min[0] - point[0], 0.0f, point[0] - max[0]});
        const float dy = std::max({min[1] - point[1], 0.0f, point[1] - max[1]});
        return std::sqrt(dx * dx + dy * dy);
    }

    bool overlaps(const BoundingBox& other) const {
        return (max[0] >= other.min[0] && other.max[0] >= min[0]) && (max[1] >= other.min[1] && other.max[1] >= min[1]);
    }

    fvec2 getMin() const {
        return min;
    }

    fvec2 getMax() const {
        return max;
    }

private:
    fvec2 min;
    fvec2 max;
};

}  // namespace dnac
