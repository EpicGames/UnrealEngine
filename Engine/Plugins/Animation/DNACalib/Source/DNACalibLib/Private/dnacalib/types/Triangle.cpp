// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/types/Triangle.h"

namespace dnac {

Triangle::Triangle(const fvec2& a_, const fvec2& b, const fvec2& c) :
    a{a_},
    v0{b - a},
    v1{c - a},
    d00{tdm::dot(v0, v0)},
    d01{tdm::dot(v0, v1)},
    d11{tdm::dot(v1, v1)},
    denom{d00 * d11 - d01 * d01} {
}

Triangle::Triangle(const std::array<fvec2, 3>& vertices) :
    Triangle(vertices[0], vertices[1], vertices[2]) {
}

fvec2 Triangle::A() const {
    return a;
}

fvec2 Triangle::B() const {
    return v0 + a;
}

fvec2 Triangle::C() const {
    return v1 + a;
}

fvec3 Triangle::getBarycentricCoords(const fvec2& point) const {
    // Real-Time Collision Detection The Morgan Kaufmann Series, chapter 3.4 Barycentric Coordinates
    const fvec2 v2 = point - a;

    const float d20 = tdm::dot(v2, v0);
    const float d21 = tdm::dot(v2, v1);

    const float v = (d11 * d20 - d01 * d21) / denom;
    const float w = (d00 * d21 - d01 * d20) / denom;
    const float u = 1.0f - v - w;
    return {u, v, w};
}

}  // namespace dnac
