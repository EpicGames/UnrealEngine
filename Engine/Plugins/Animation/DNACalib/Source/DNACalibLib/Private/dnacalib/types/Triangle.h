// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"

#include <array>

namespace dnac {

class Triangle {

public:
    Triangle(const fvec2& a, const fvec2& b, const fvec2& c);
    explicit Triangle(const std::array<fvec2, 3>& vertices);

    fvec3 getBarycentricCoords(const fvec2& point) const;

    fvec2 A() const;
    fvec2 B() const;
    fvec2 C() const;

private:
    fvec2 a;
    fvec2 v0;     // b-a
    fvec2 v1;     // c-a;
    float d00;    // v0 dot v0
    float d01;    // v0 dot v1
    float d11;    // v1 dot v1
    float denom;  // d00 * d11 - d01 * d01;
};

}  // namespace dnac
