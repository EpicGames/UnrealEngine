// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/types/UVBarycentricMapping.h"

namespace dnac {

UVBarycentricMapping::UVBarycentricMapping(const std::function<ConstArrayView<std::uint32_t>(std::uint32_t)>& faceGetter,
                                           ConstArrayView<std::uint32_t> vertexPositionIndices,
                                           ConstArrayView<std::uint32_t> textureCoordinateUVIndices,
                                           ConstArrayView<float> Us,
                                           ConstArrayView<float> Vs,
                                           std::uint32_t faceCount,
                                           MemoryResource* memRes) :
    triangles{memRes},
    trianglePositionIndices{memRes} {

    auto estimatedTriangleCount = static_cast<std::size_t>(static_cast<float>(faceCount) * 2.5f);
    triangles.reserve(estimatedTriangleCount);
    boundingBoxes.reserve(estimatedTriangleCount);
    trianglePositionIndices.reserve(estimatedTriangleCount);

    for (std::uint32_t fi = 0u; fi < faceCount; fi++) {
        auto face = faceGetter(fi);
        if (face.size() > 2) {
            const auto n = face.size();
            for (std::uint32_t i = 0; i < n - 2; ++i) {
                for (std::uint32_t j = i + 1; j < n - 1; ++j) {
                    for (std::uint32_t k = j + 1; k < n; ++k) {
                        const auto vli0 = face[i];
                        const auto vli1 = face[j];
                        const auto vli2 = face[k];
                        const std::array<std::uint32_t, 3> positionIndices{vertexPositionIndices[vli0],
                                                                           vertexPositionIndices[vli1],
                                                                           vertexPositionIndices[vli2]};
                        const auto uvIndex0 = textureCoordinateUVIndices[vli0];
                        const auto uvIndex1 = textureCoordinateUVIndices[vli1];
                        const auto uvIndex2 = textureCoordinateUVIndices[vli2];

                        const std::array<fvec2, 3> UVs = {fvec2{Us[uvIndex0], Vs[uvIndex0]},
                                                          fvec2{Us[uvIndex1], Vs[uvIndex1]},
                                                          fvec2{Us[uvIndex2], Vs[uvIndex2]}};
                        triangles.emplace_back(UVs);
                        boundingBoxes.emplace_back(UVs);
                        trianglePositionIndices.emplace_back(positionIndices);
                    }
                }
            }
        }
    }
    triangles.shrink_to_fit();
    boundingBoxes.shrink_to_fit();
    trianglePositionIndices.shrink_to_fit();
}

UVBarycentricMapping::BarycentricPositionIndicesPair UVBarycentricMapping::getBarycentric(fvec2 uv) const {
    const auto isPointInsideTriangle = [](const fvec3& barycentricPoint) {
        return barycentricPoint[0] > 0.0f && barycentricPoint[1] > 0.0f && barycentricPoint[2] > 0.0f;
    };
    for (std::uint32_t i = 0; i < triangles.size(); i++) {
        const auto& triangle = triangles[i];
        // we check if point is inside triangle (all barycentric coordinates are positive)
        if (boundingBoxes[i].contains(uv)) {
            const auto barycentricPoint = triangle.getBarycentricCoords(uv);
            if (isPointInsideTriangle(barycentricPoint)) {
                return BarycentricPositionIndicesPair{barycentricPoint,
                                                      ConstArrayView<std::uint32_t>{trianglePositionIndices[i]}};
            }
        }
    }
    return {};
}

const Triangle& UVBarycentricMapping::getTriangle(std::uint32_t index) const {
    return triangles[index];
}

ConstArrayView<std::uint32_t> UVBarycentricMapping::getTrianglePositionIndices(std::uint32_t index) const {
    return trianglePositionIndices[index];
}

ConstArrayView<BoundingBox> UVBarycentricMapping::getBoundingBoxes() const {
    return boundingBoxes;
}

}  // namespace dnac
