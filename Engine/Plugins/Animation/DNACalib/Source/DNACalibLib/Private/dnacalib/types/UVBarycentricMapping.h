// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/types/Aliases.h"
#include "dnacalib/types/BoundingBox.h"
#include "dnacalib/types/Triangle.h"

namespace dnac {

class UVBarycentricMapping {
public:
    using TrianglePositionIndicesPair = std::tuple<Triangle, std::array<std::uint32_t, 3u>>;
    using BarycentricPositionIndicesPair = std::tuple<fvec3, ConstArrayView<std::uint32_t>>;

public:
    UVBarycentricMapping(const std::function<ConstArrayView<std::uint32_t>(std::uint32_t)>& faceGetter,
                         ConstArrayView<std::uint32_t> vertexPositionIndices,
                         ConstArrayView<std::uint32_t> textureCoordinateUVIndices,
                         ConstArrayView<float> Us,
                         ConstArrayView<float> Vs,
                         std::uint32_t faceCount,
                         MemoryResource* memRes);

    BarycentricPositionIndicesPair getBarycentric(fvec2 uv) const;
    const Triangle& getTriangle(std::uint32_t index) const;
    ConstArrayView<std::uint32_t> getTrianglePositionIndices(std::uint32_t index) const;
    ConstArrayView<BoundingBox> getBoundingBoxes() const;

private:
    Vector<Triangle> triangles;
    Vector<BoundingBox> boundingBoxes;
    Vector<std::array<std::uint32_t, 3u>> trianglePositionIndices;
};

}  // namespace dnac
