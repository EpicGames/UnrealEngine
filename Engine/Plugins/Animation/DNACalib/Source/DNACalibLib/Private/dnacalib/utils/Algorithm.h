// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"

namespace dnac {

inline fmat4 extractTranslationMatrix(const fmat4& transformationMatrix) {
    auto t = fmat4::identity();
    t(3, 0) = transformationMatrix(3, 0);
    t(3, 1) = transformationMatrix(3, 1);
    t(3, 2) = transformationMatrix(3, 2);
    return t;
}

inline fvec3 extractScaleVector(const fmat4& transformationMatrix) {
    const auto& m = transformationMatrix;
    const float sx = fvec3{m(0, 0), m(0, 1), m(0, 2)}.length();
    const float sy = fvec3{m(1, 0), m(1, 1), m(1, 2)}.length();
    const float sz = fvec3{m(2, 0), m(2, 1), m(2, 2)}.length();
    return {sx, sy, sz};
}

inline fmat4 extractRotationMatrix(const fmat4& transformationMatrix) {
    auto r = transformationMatrix;
    r(3, 0) = 0.0f;
    r(3, 1) = 0.0f;
    r(3, 2) = 0.0f;
    const auto scale = tdm::scale(extractScaleVector(transformationMatrix));
    const auto inverseScale = tdm::inverse(scale);
    return inverseScale * r;
}

inline fvec3 extractTranslationVector(const fmat4& transformationMatrix) {
    return {transformationMatrix(3, 0), transformationMatrix(3, 1), transformationMatrix(3, 2)};
}

inline frad3 extractRotationVector(const fmat4& transformationMatrix) {
    frad3 angle{};
    const auto r = extractRotationMatrix(transformationMatrix);
    const auto r02 = r(0, 2);
    if (r02 < 1.0f) {
        if (r02 > -1.0f) {
            angle[0] = tdm::frad{std::atan2(r(1, 2), r(2, 2))};
            angle[1] = tdm::frad{std::asin(-r02)};
            angle[2] = tdm::frad{std::atan2(r(0, 1), r(0, 0))};
        } else {
            angle[0] = tdm::frad{std::atan2(-r(2, 1), r(1, 1))};
            angle[1] = tdm::frad{static_cast<float>(tdm::pi() / 2.0f)};
            angle[2] = tdm::frad{0};
        }
    } else {
        angle[0] = tdm::frad{-std::atan2(-r(2, 1), r(1, 1))};
        angle[1] = tdm::frad{static_cast<float>(-tdm::pi() / 2.0f)};
        angle[2] = tdm::frad{0};
    }
    return angle;
}

}  // namespace dnac
