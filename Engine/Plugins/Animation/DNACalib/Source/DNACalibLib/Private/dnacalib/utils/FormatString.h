// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/TypeDefs.h"

namespace dnac {

template<typename... Args>
String<char> formatString(MemoryResource* memRes, const char* format, Args&&... args) {
    String<char> result{memRes};
    result.resize(1024);
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wformat-security"
#endif
    // The returned number of bytes to be written does not include the null terminator
    const auto neededSize = snprintf(nullptr, 0ul, format, args...) + 1;
    const auto size = std::min(result.size(), static_cast<std::size_t>(neededSize));
    snprintf(&result[0], size, format, args...);
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
    result.resize(size);
    return result;
}

}  // namespace dnac
