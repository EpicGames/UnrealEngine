// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/version/VersionInfo.h"

#include "dnacalib/version/Version.h"

#include <cstring>

namespace dnac {

namespace {

constexpr int majorVersion = DNAC_MAJOR_VERSION;
constexpr int minorVersion = DNAC_MINOR_VERSION;
constexpr int patchVersion = DNAC_PATCH_VERSION;
constexpr const char* versionString = DNAC_VERSION_STRING;

}  // namespace

int VersionInfo::getMajorVersion() {
    return majorVersion;
}

int VersionInfo::getMinorVersion() {
    return minorVersion;
}

int VersionInfo::getPatchVersion() {
    return patchVersion;
}

StringView VersionInfo::getVersionString() {
    return {versionString, std::strlen(versionString)};
}

}  // namespace dnac
