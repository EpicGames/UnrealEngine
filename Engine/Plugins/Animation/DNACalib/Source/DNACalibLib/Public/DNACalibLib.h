// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef DNACALIB_MODULE_DISCARD

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDNACalibLib, Log, All);

class FDNACalibLib : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

#endif  // DNACALIB_MODULE_DISCARD
