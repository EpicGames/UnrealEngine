// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class DNACalibDNAReader;

/**
    @brief Command is an abstract class whose implementations are expected to modify the DNA provided in the run() method in some
   way.
*/
class DNACAPI Command {
public:
    virtual ~Command();
    virtual void run(DNACalibDNAReader* output) = 0;
};

}  // namespace dnac
