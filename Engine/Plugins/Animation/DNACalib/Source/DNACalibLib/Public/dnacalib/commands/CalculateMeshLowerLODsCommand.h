// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class DNACalibDNAReader;

/**
    @brief CalculateMeshLowerLODsCommand is used to recalculate vertex positions for lower LOD meshes of the specified mesh.
    @note
        The calculation is done based on vertex positions of the specified mesh and vertex texture coordinates of its lower LOD
   meshes.
*/
class CalculateMeshLowerLODsCommand : public Command {
public:
    DNACAPI explicit CalculateMeshLowerLODsCommand(MemoryResource* memRes = nullptr);

    DNACAPI explicit CalculateMeshLowerLODsCommand(std::uint16_t meshIndex, MemoryResource* memRes = nullptr);

    DNACAPI ~CalculateMeshLowerLODsCommand();

    CalculateMeshLowerLODsCommand(const CalculateMeshLowerLODsCommand&) = delete;
    CalculateMeshLowerLODsCommand& operator=(const CalculateMeshLowerLODsCommand&) = delete;

    DNACAPI CalculateMeshLowerLODsCommand(CalculateMeshLowerLODsCommand&&);
    DNACAPI CalculateMeshLowerLODsCommand& operator=(CalculateMeshLowerLODsCommand&&);

    /**
        @brief Method for setting the index of the mesh to calculate lower LOD meshes from.
        @param meshIndex
            The index of the mesh.
    */
    DNACAPI void setMeshIndex(std::uint16_t meshIndex);

    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
