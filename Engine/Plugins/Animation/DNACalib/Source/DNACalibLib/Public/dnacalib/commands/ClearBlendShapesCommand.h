// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief ClearBlendShapesCommand is used to clear all blend shapes data from a DNA.
    @note This command clears blend shape target deltas and blend shape animation data. By doing so, it transforms the DNA to be
   "joints only".
*/
class ClearBlendShapesCommand : public Command {
public:
    DNACAPI explicit ClearBlendShapesCommand(MemoryResource* memRes = nullptr);

    DNACAPI ~ClearBlendShapesCommand();

    ClearBlendShapesCommand(const ClearBlendShapesCommand&) = delete;
    ClearBlendShapesCommand& operator=(const ClearBlendShapesCommand&) = delete;

    DNACAPI ClearBlendShapesCommand(ClearBlendShapesCommand&&);
    DNACAPI ClearBlendShapesCommand& operator=(ClearBlendShapesCommand&&);

    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
