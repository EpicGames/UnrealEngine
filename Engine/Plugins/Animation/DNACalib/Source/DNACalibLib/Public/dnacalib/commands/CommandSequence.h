// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class DNACalibDNAReader;

/**
    @brief CommandSequence is used to run a sequence of commands on the same DNA.
    @note
        Commands will be run in the order in which they were added to the sequence.
    @note
        CommandSequence holds pointers to commands, but does not own them.
*/
class CommandSequence : public Command {
public:
    DNACAPI explicit CommandSequence(MemoryResource* memRes = nullptr);

    DNACAPI ~CommandSequence();

    CommandSequence(const CommandSequence&) = delete;
    CommandSequence& operator=(const CommandSequence&) = delete;

    DNACAPI CommandSequence(CommandSequence&&);
    DNACAPI CommandSequence& operator=(CommandSequence&&);

    DNACAPI void run(DNACalibDNAReader* output) override;

    /**
        @brief Method for adding a command to a sequence of commands to run.
        @param command
            The command to add.
    */
    DNACAPI void add(Command* command);

    /**
        @brief Method for adding multiple commands to a sequence of commands to run.
        @param commands
            The commands to add.
    */
    DNACAPI void add(ArrayView<Command> commands);

    template<class... Commands>
    void add(Commands... commands) {
        static_assert(sizeof...(commands) > 0, "At least one command must be passed.");
        Command* commandList[] = {commands...};
        for (auto cmd : commandList) {
            add(cmd);
        }
    }

    /**
        @brief Method for removing a command from the sequence of commands to run.
        @param command
            The command to remove.
    */
    DNACAPI void remove(Command* command);

    /**
        @brief Method for removing an array of commands from the sequence of commands to run.
        @param commands
            The commands to remove.
    */
    DNACAPI void remove(ArrayView<Command> commands);

    template<class... Commands>
    void remove(Commands... commands) {
        static_assert(sizeof...(commands) > 0, "At least one command must be passed.");
        Command* commandList[] = {commands...};
        for (auto cmd : commandList) {
            remove(cmd);
        }
    }

    /**
        @brief Method for checking if the provided command is part of the command sequence.
        @param command
            The command to check.
    */
    DNACAPI bool contains(Command* command) const;

    /**
        @brief Number of commands in command sequence.
    */
    DNACAPI std::size_t size() const;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
