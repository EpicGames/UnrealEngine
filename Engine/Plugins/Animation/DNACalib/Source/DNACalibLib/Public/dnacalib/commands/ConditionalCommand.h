// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"

namespace dnac {

class DNACalibDNAReader;

/**
    @brief ConditionalCommand is used to run a command if the specified condition is met.
*/
template<class TCommand, class TCondition>
class ConditionalCommand : public Command {
private:
    using CommandType = TCommand;
    using ConditionType = TCondition;

public:
    ConditionalCommand() :
        command{nullptr},
        condition{} {
    }

    ConditionalCommand(CommandType* command_, ConditionType condition_) :
        command{command_},
        condition{condition_} {
    }

    ~ConditionalCommand() = default;

    ConditionalCommand(const ConditionalCommand&) = delete;
    ConditionalCommand& operator=(const ConditionalCommand&) = delete;

    ConditionalCommand(ConditionalCommand&&) = default;
    ConditionalCommand& operator=(ConditionalCommand&&) = default;

    /**
        @brief Method for setting the command to run.
        @param command_
            The command to run.
    */
    void setCommand(Command* command_) {
        command = command_;
    }

    /**
        @brief Method for setting the condition under which the command should run.
        @param condition_
            The condition that should be met.
    */
    void setCondition(ConditionType condition_) {
        condition = condition_;
    }

    void run(DNACalibDNAReader* output) override {
        if (command && condition(command, output)) {
            command->run(output);
        }
    }

private:
    CommandType* command;
    ConditionType condition;
};

template<class TCommand, class TCondition>
ConditionalCommand<TCommand, TCondition> makeConditional(TCommand* command, TCondition condition) {
    return ConditionalCommand<TCommand, TCondition>{command, condition};
}

}  // namespace dnac
