// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class DNACalibDNAReader;

/**
    @brief ConvertUnitsCommand is used to perform unit conversions over all relevant data in a DNA.
*/
class ConvertUnitsCommand : public Command {
public:
    DNACAPI explicit ConvertUnitsCommand(MemoryResource* memRes = nullptr);
    DNACAPI explicit ConvertUnitsCommand(TranslationUnit translationUnit,
                                         RotationUnit rotationUnit,
                                         MemoryResource* memRes = nullptr);

    DNACAPI ~ConvertUnitsCommand();

    ConvertUnitsCommand(const ConvertUnitsCommand&) = delete;
    ConvertUnitsCommand& operator=(const ConvertUnitsCommand&) = delete;

    DNACAPI ConvertUnitsCommand(ConvertUnitsCommand&&);
    DNACAPI ConvertUnitsCommand& operator=(ConvertUnitsCommand&&);

    DNACAPI void setTranslationUnit(TranslationUnit translationUnit);
    DNACAPI void setRotationUnit(RotationUnit rotationUnit);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
