// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief PruneBlendShapeTargetsCommand is used to prune blend shape target deltas whose absolute magnitude is less than or equal
   to the specified threshold.
*/
class PruneBlendShapeTargetsCommand : public Command {
public:
    DNACAPI explicit PruneBlendShapeTargetsCommand(MemoryResource* memRes = nullptr);
    DNACAPI explicit PruneBlendShapeTargetsCommand(float threshold, MemoryResource* memRes = nullptr);

    DNACAPI ~PruneBlendShapeTargetsCommand();

    PruneBlendShapeTargetsCommand(const PruneBlendShapeTargetsCommand&) = delete;
    PruneBlendShapeTargetsCommand& operator=(const PruneBlendShapeTargetsCommand&) = delete;

    DNACAPI PruneBlendShapeTargetsCommand(PruneBlendShapeTargetsCommand&&);
    DNACAPI PruneBlendShapeTargetsCommand& operator=(PruneBlendShapeTargetsCommand&&);

    /**
        @brief Method for setting the threshold for pruning blend shape target deltas.
        @param threshold
            The threshold to use.
    */
    DNACAPI void setThreshold(float threshold);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
