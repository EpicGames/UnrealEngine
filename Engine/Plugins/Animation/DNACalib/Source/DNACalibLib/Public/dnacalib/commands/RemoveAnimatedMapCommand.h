// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief RemoveAnimatedMapCommand is used to remove animated maps.
*/
class RemoveAnimatedMapCommand : public Command {
public:
    DNACAPI explicit RemoveAnimatedMapCommand(MemoryResource* memRes = nullptr);
    DNACAPI RemoveAnimatedMapCommand(std::uint16_t animatedMapIndex, MemoryResource* memRes = nullptr);
    DNACAPI RemoveAnimatedMapCommand(ConstArrayView<std::uint16_t> animatedMapIndices, MemoryResource* memRes = nullptr);

    DNACAPI ~RemoveAnimatedMapCommand();

    RemoveAnimatedMapCommand(const RemoveAnimatedMapCommand&) = delete;
    RemoveAnimatedMapCommand& operator=(const RemoveAnimatedMapCommand&) = delete;

    DNACAPI RemoveAnimatedMapCommand(RemoveAnimatedMapCommand&&);
    DNACAPI RemoveAnimatedMapCommand& operator=(RemoveAnimatedMapCommand&&);

    /**
        @brief Method for setting the index of the animated map to remove.
        @param animatedMapIndex
            The index of the animated map.
        @note Call to either setter overwrites previous setter calls. When running the command, the last set animated map(s) will
       be removed.
    */
    DNACAPI void setAnimatedMapIndex(std::uint16_t animatedMapIndex);
    /**
        @brief Method for setting the indices of animated maps to remove.
        @param animatedMapIndices
            The animated map indices.
        @note Call to either setter overwrites previous setter calls. When running the command, the last set animated map(s) will
       be removed.
    */
    DNACAPI void setAnimatedMapIndices(ConstArrayView<std::uint16_t> animatedMapIndices);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
