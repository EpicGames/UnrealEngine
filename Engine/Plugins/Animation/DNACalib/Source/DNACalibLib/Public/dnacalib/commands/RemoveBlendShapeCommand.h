// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief RemoveBlendShapeCommand is used to remove blend shapes.
*/
class RemoveBlendShapeCommand : public Command {
public:
    DNACAPI explicit RemoveBlendShapeCommand(MemoryResource* memRes = nullptr);
    DNACAPI RemoveBlendShapeCommand(std::uint16_t blendShapeIndex, MemoryResource* memRes = nullptr);
    DNACAPI RemoveBlendShapeCommand(ConstArrayView<std::uint16_t> blendShapeIndices, MemoryResource* memRes = nullptr);

    DNACAPI ~RemoveBlendShapeCommand();

    RemoveBlendShapeCommand(const RemoveBlendShapeCommand&) = delete;
    RemoveBlendShapeCommand& operator=(const RemoveBlendShapeCommand&) = delete;

    DNACAPI RemoveBlendShapeCommand(RemoveBlendShapeCommand&&);
    DNACAPI RemoveBlendShapeCommand& operator=(RemoveBlendShapeCommand&&);

    /**
        @brief Method for setting the index of the blend shape to remove.
        @param blendShapeIndex
            The index of the blend shape.
        @note Call to either setter overwrites previous setter calls. When running the command, the last set blend shape(s) will
       be removed.
    */
    DNACAPI void setBlendShapeIndex(std::uint16_t blendShapeIndex);
    /**
        @brief Method for setting the indices of blend shapes to remove.
        @param blendShapeIndices
            The blend shape indices.
        @note Call to either setter overwrites previous setter calls. When running the command, the last set blend shape(s) will
       be removed.
    */
    DNACAPI void setBlendShapeIndices(ConstArrayView<std::uint16_t> blendShapeIndices);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
