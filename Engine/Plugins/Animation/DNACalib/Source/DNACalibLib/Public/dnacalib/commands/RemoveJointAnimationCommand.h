// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief RemoveJointAnimationCommand is used to remove joint animation data.
*/
class RemoveJointAnimationCommand : public Command {
public:
    DNACAPI explicit RemoveJointAnimationCommand(MemoryResource* memRes = nullptr);
    DNACAPI RemoveJointAnimationCommand(std::uint16_t jointIndex, MemoryResource* memRes = nullptr);
    DNACAPI RemoveJointAnimationCommand(ConstArrayView<std::uint16_t> jointIndices, MemoryResource* memRes = nullptr);

    DNACAPI ~RemoveJointAnimationCommand();

    RemoveJointAnimationCommand(const RemoveJointAnimationCommand&) = delete;
    RemoveJointAnimationCommand& operator=(const RemoveJointAnimationCommand&) = delete;

    DNACAPI RemoveJointAnimationCommand(RemoveJointAnimationCommand&&);
    DNACAPI RemoveJointAnimationCommand& operator=(RemoveJointAnimationCommand&&);

    /**
        @brief Method for setting the index of a joint whose animation data to remove.
        @param jointIndex
            The index of the joint.
        @note Call to either setter overwrites previous setter calls. When running the command, the last set joint animation(s)
       will be removed.
    */
    DNACAPI void setJointIndex(std::uint16_t jointIndex);
    /**
        @brief Method for setting the indices of joints whose animation data to remove.
        @param jointIndices
            The joint indices.
        @note Call to either setter overwrites previous setter calls. When running the command, the last set joint animation(s)
       will be removed.
    */
    DNACAPI void setJointIndices(ConstArrayView<std::uint16_t> jointIndices);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
