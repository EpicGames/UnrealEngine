// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief RemoveJointCommand is used to remove joints.
*/
class RemoveJointCommand : public Command {
public:
    DNACAPI explicit RemoveJointCommand(MemoryResource* memRes = nullptr);
    DNACAPI RemoveJointCommand(std::uint16_t jointIndex, MemoryResource* memRes = nullptr);
    DNACAPI RemoveJointCommand(ConstArrayView<std::uint16_t> jointIndices, MemoryResource* memRes = nullptr);

    DNACAPI ~RemoveJointCommand();

    RemoveJointCommand(const RemoveJointCommand&) = delete;
    RemoveJointCommand& operator=(const RemoveJointCommand&) = delete;

    DNACAPI RemoveJointCommand(RemoveJointCommand&&);
    DNACAPI RemoveJointCommand& operator=(RemoveJointCommand&&);

    /**
        @brief Method for setting the index of the joint to remove.
        @param jointIndex
            The index of the joint.
        @note Call to either setter overwrites previous setter calls. When running the command, the last set joint(s) will be
       removed.
    */
    DNACAPI void setJointIndex(std::uint16_t jointIndex);
    /**
        @brief Method for setting the indices of joints to remove.
        @param jointIndices
            The joint indices.
        @note Call to either setter overwrites previous setter calls. When running the command, the last set joint(s) will be
       removed.
    */
    DNACAPI void setJointIndices(ConstArrayView<std::uint16_t> jointIndices);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
