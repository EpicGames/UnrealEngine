// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief RemoveMeshCommand is used to remove meshes.
*/
class RemoveMeshCommand : public Command {
public:
    DNACAPI explicit RemoveMeshCommand(MemoryResource* memRes = nullptr);
    DNACAPI RemoveMeshCommand(std::uint16_t meshIndex, MemoryResource* memRes = nullptr);
    DNACAPI RemoveMeshCommand(ConstArrayView<std::uint16_t> meshIndices, MemoryResource* memRes = nullptr);

    DNACAPI ~RemoveMeshCommand();

    RemoveMeshCommand(const RemoveMeshCommand&) = delete;
    RemoveMeshCommand& operator=(const RemoveMeshCommand&) = delete;

    DNACAPI RemoveMeshCommand(RemoveMeshCommand&&);
    DNACAPI RemoveMeshCommand& operator=(RemoveMeshCommand&&);

    /**
        @brief Method for setting the index of the mesh to remove.
        @param meshIndex
            The index of the mesh.
    */
    DNACAPI void setMeshIndex(std::uint16_t meshIndex);
    /**
        @brief Method for setting the indices of meshes to remove.
        @param meshIndices
            The mesh indices.
        @note Call to either setter overwrites previous setter calls. When running the command, the last set mesh(es) will be
       removed.
    */
    DNACAPI void setMeshIndices(ConstArrayView<std::uint16_t> meshIndices);

    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
