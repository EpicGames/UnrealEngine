// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief RenameAnimatedMapCommand is used to rename an animated map.
*/
class RenameAnimatedMapCommand : public Command {
public:
    DNACAPI explicit RenameAnimatedMapCommand(MemoryResource* memRes = nullptr);
    DNACAPI RenameAnimatedMapCommand(std::uint16_t animatedMapIndex, const char* newName, MemoryResource* memRes = nullptr);
    DNACAPI RenameAnimatedMapCommand(const char* oldName, const char* newName, MemoryResource* memRes = nullptr);

    DNACAPI ~RenameAnimatedMapCommand();

    RenameAnimatedMapCommand(const RenameAnimatedMapCommand&) = delete;
    RenameAnimatedMapCommand& operator=(const RenameAnimatedMapCommand&) = delete;

    DNACAPI RenameAnimatedMapCommand(RenameAnimatedMapCommand&&);
    DNACAPI RenameAnimatedMapCommand& operator=(RenameAnimatedMapCommand&&);

    /**
        @brief Method for setting a new name for animated map with given index.
        @param animatedMapIndex
            The index of the animated map whose name to change.
        @param newName
            The new name for the animated map.
    */
    DNACAPI void setName(std::uint16_t animatedMapIndex, const char* newName);

    /**
        @brief Method for setting a new name for animated map with given name.
        @note
            The renaming will not happen if there is no animated map with given current name.
        @param oldName
            The current name of the animated map whose name to change.
        @param newName
            The new name for the animated map.
    */
    DNACAPI void setName(const char* oldName, const char* newName);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
