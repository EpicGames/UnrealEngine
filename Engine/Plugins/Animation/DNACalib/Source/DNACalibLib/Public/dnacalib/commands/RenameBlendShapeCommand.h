// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief RenameBlendShapeCommand is used to rename a blend shape channel.
*/
class RenameBlendShapeCommand : public Command {
public:
    DNACAPI explicit RenameBlendShapeCommand(MemoryResource* memRes = nullptr);
    DNACAPI RenameBlendShapeCommand(std::uint16_t blendShapeIndex, const char* newName, MemoryResource* memRes = nullptr);
    DNACAPI RenameBlendShapeCommand(const char* oldName, const char* newName, MemoryResource* memRes = nullptr);

    DNACAPI ~RenameBlendShapeCommand();

    RenameBlendShapeCommand(const RenameBlendShapeCommand&) = delete;
    RenameBlendShapeCommand& operator=(const RenameBlendShapeCommand&) = delete;

    DNACAPI RenameBlendShapeCommand(RenameBlendShapeCommand&&);
    DNACAPI RenameBlendShapeCommand& operator=(RenameBlendShapeCommand&&);

    /**
        @brief Method for setting a new name for blend shape channel with given index.
        @param blendShapeIndex
            The index of the blend shape channel whose name to change.
        @param newName
            The new name for the blend shape channel.
    */
    DNACAPI void setName(std::uint16_t blendShapeIndex, const char* newName);

    /**
        @brief Method for setting a new name for blend shape channel with given name.
        @note
            The renaming will not happen if there is no blend shape channel with given current name.
        @param oldName
            The current name of the blend shape channel whose name to change.
        @param newName
            The new name for the blend shape channel.
    */
    DNACAPI void setName(const char* oldName, const char* newName);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
