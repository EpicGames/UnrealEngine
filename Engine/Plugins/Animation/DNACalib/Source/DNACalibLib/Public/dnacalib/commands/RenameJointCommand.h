// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief RenameJointCommand is used to rename a joint.
*/
class RenameJointCommand : public Command {
public:
    DNACAPI explicit RenameJointCommand(MemoryResource* memRes = nullptr);
    DNACAPI RenameJointCommand(std::uint16_t jointIndex, const char* newName, MemoryResource* memRes = nullptr);
    DNACAPI RenameJointCommand(const char* oldName, const char* newName, MemoryResource* memRes = nullptr);

    DNACAPI ~RenameJointCommand();

    RenameJointCommand(const RenameJointCommand&) = delete;
    RenameJointCommand& operator=(const RenameJointCommand&) = delete;

    DNACAPI RenameJointCommand(RenameJointCommand&&);
    DNACAPI RenameJointCommand& operator=(RenameJointCommand&&);

    /**
        @brief Method for setting a new name for joint with given index.
        @param jointIndex
            The index of the joint whose name to change.
        @param newName
            The new name for the joint.
    */
    DNACAPI void setName(std::uint16_t jointIndex, const char* newName);

    /**
        @brief Method for setting a new name for joint with given name.
        @note
            The renaming will not happen if there is no joint with given current name.
        @param oldName
            The current name of the joint whose name to change.
        @param newName
            The new name for the joint.
    */
    DNACAPI void setName(const char* oldName, const char* newName);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
