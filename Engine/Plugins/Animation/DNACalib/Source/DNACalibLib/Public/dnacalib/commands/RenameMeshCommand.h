// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief RenameMeshCommand is used to rename a mesh.
*/
class RenameMeshCommand : public Command {
public:
    DNACAPI explicit RenameMeshCommand(MemoryResource* memRes = nullptr);
    DNACAPI RenameMeshCommand(std::uint16_t meshIndex, const char* newName, MemoryResource* memRes = nullptr);
    DNACAPI RenameMeshCommand(const char* oldName, const char* newName, MemoryResource* memRes = nullptr);

    DNACAPI ~RenameMeshCommand();

    RenameMeshCommand(const RenameMeshCommand&) = delete;
    RenameMeshCommand& operator=(const RenameMeshCommand&) = delete;

    DNACAPI RenameMeshCommand(RenameMeshCommand&&);
    DNACAPI RenameMeshCommand& operator=(RenameMeshCommand&&);

    /**
        @brief Method for setting a new name for mesh with given index.
        @param meshIndex
            The index of the mesh whose name to change.
        @param newName
            The new name for the mesh.
    */
    DNACAPI void setName(std::uint16_t meshIndex, const char* newName);

    /**
        @brief Method for setting a new name for mesh with given name.
        @note
            The renaming will not happen if there is no mesh with given current name.
        @param oldName
            The current name of the mesh whose name to change.
        @param newName
            The new name for the mesh.
    */
    DNACAPI void setName(const char* oldName, const char* newName);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
