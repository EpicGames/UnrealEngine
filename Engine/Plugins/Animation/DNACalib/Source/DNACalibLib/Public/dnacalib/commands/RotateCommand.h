// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class DNACalibDNAReader;

/**
    @brief RotateCommand is used to rotate neutral joints and vertex positions around given origin.
    @note
        Joint rotations are represented in parent space, so it is enough to rotate only root joints, as that rotation will be
   propagated to the rest of the joints.
    @note
        If the origin is not set, the assumed origin is (0, 0, 0).
*/
class RotateCommand : public Command {
public:
    DNACAPI explicit RotateCommand(MemoryResource* memRes = nullptr);
    DNACAPI RotateCommand(Vector3 degrees, Vector3 origin, MemoryResource* memRes = nullptr);

    DNACAPI ~RotateCommand();

    RotateCommand(const RotateCommand&) = delete;
    RotateCommand& operator=(const RotateCommand&) = delete;

    DNACAPI RotateCommand(RotateCommand&&);
    DNACAPI RotateCommand& operator=(RotateCommand&&);

    /**
        @brief Method for setting the rotation angles.
        @param degrees
            Rotation angles in degrees.
    */
    DNACAPI void setRotation(Vector3 degrees);

    /**
        @brief Method for setting the rotation origin.
        @param origin
            Origin coordinates.
    */
    DNACAPI void setOrigin(Vector3 origin);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
