// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class DNACalibDNAReader;

/**
    @brief ScaleCommand is used to scale neutral joints, vertex positions and joint and blendshape deltas by a factor.
    @note
        Only translation attributes of neutral joints and joint deltas are scaled.
*/
class ScaleCommand : public Command {
public:
    DNACAPI explicit ScaleCommand(MemoryResource* memRes = nullptr);
    DNACAPI ScaleCommand(float scale, Vector3 origin, MemoryResource* memRes = nullptr);

    DNACAPI ~ScaleCommand();

    ScaleCommand(const ScaleCommand&) = delete;
    ScaleCommand& operator=(const ScaleCommand&) = delete;

    DNACAPI ScaleCommand(ScaleCommand&&);
    DNACAPI ScaleCommand& operator=(ScaleCommand&&);

    /**
        @brief Method for setting the scale factor to multiply with.
        @param scale
            Scale factor.
    */
    DNACAPI void setScale(float scale);

    /**
        @brief Method for setting the origin.
        @note The origin is used to properly scale position values (vertex positions and neutral joint translations).
        @param origin
            Origin coordinates.
    */
    DNACAPI void setOrigin(Vector3 origin);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
