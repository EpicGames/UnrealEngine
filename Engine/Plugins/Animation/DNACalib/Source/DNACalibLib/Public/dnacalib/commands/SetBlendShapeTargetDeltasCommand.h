// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/commands/VectorOperations.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief SetBlendShapeTargetDeltasCommand is used to change blend shape target deltas.
*/
class SetBlendShapeTargetDeltasCommand : public Command {
public:
    DNACAPI static const sc::StatusCode VertexIndicesOutOfBoundsError;
    DNACAPI static const sc::StatusCode NoVertexIndicesSetError;
    DNACAPI static const sc::StatusCode DeltasVertexIndicesCountMismatch;
    DNACAPI static const sc::StatusCode DeltasMasksCountMismatch;

public:
    DNACAPI explicit SetBlendShapeTargetDeltasCommand(MemoryResource* memRes = nullptr);
    DNACAPI SetBlendShapeTargetDeltasCommand(std::uint16_t meshIndex,
                                             std::uint16_t blendShapeTargetIndex,
                                             ConstArrayView<Vector3> deltas,
                                             ConstArrayView<std::uint32_t> vertexIndices,
                                             VectorOperation operation,
                                             MemoryResource* memRes = nullptr);
    DNACAPI SetBlendShapeTargetDeltasCommand(std::uint16_t meshIndex,
                                             std::uint16_t blendShapeTargetIndex,
                                             ConstArrayView<float> xs,
                                             ConstArrayView<float> ys,
                                             ConstArrayView<float> zs,
                                             ConstArrayView<std::uint32_t> vertexIndices,
                                             VectorOperation operation,
                                             MemoryResource* memRes = nullptr);
    DNACAPI SetBlendShapeTargetDeltasCommand(std::uint16_t meshIndex,
                                             std::uint16_t blendShapeTargetIndex,
                                             ConstArrayView<Vector3> deltas,
                                             ConstArrayView<std::uint32_t> vertexIndices,
                                             ConstArrayView<float> masks,
                                             VectorOperation operation,
                                             MemoryResource* memRes = nullptr);
    DNACAPI SetBlendShapeTargetDeltasCommand(std::uint16_t meshIndex,
                                             std::uint16_t blendShapeTargetIndex,
                                             ConstArrayView<float> xs,
                                             ConstArrayView<float> ys,
                                             ConstArrayView<float> zs,
                                             ConstArrayView<std::uint32_t> vertexIndices,
                                             ConstArrayView<float> masks,
                                             VectorOperation operation,
                                             MemoryResource* memRes = nullptr);

    DNACAPI ~SetBlendShapeTargetDeltasCommand();

    SetBlendShapeTargetDeltasCommand(const SetBlendShapeTargetDeltasCommand&) = delete;
    SetBlendShapeTargetDeltasCommand& operator=(const SetBlendShapeTargetDeltasCommand&) = delete;

    DNACAPI SetBlendShapeTargetDeltasCommand(SetBlendShapeTargetDeltasCommand&&);
    DNACAPI SetBlendShapeTargetDeltasCommand& operator=(SetBlendShapeTargetDeltasCommand&&);

    /**
        @brief Method for setting the index of the mesh whose blend shape target to change.
        @param meshIndex
            The mesh index.
    */
    DNACAPI void setMeshIndex(std::uint16_t meshIndex);
    /**
        @brief Method for setting the index of the blend shape target to change.
        @param blendShapeTargetIndex
            The blend shape target index.
    */
    DNACAPI void setBlendShapeTargetIndex(std::uint16_t blendShapeTargetIndex);

    /**
        @brief Method for setting the values used to calculate new deltas for blend shape target.
        @param deltas
            The values used in calculation.
    */
    DNACAPI void setDeltas(ConstArrayView<Vector3> deltas);

    /**
        @brief Method for setting the values used to calculate new deltas for blend shape target.
        @param xs
            The X values for each delta.
        @param ys
            The Y values for each delta.
        @param zs
            The Z values for each delta.
    */
    DNACAPI void setDeltas(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs);

    /**
        @brief Method for setting the vertex indices that correspond to new deltas.
        @param vertexIndices
            The vertexIndices.
    */
    DNACAPI void setVertexIndices(ConstArrayView<std::uint32_t> vertexIndices);

    /**
        @brief Method for setting masks used to calculate new deltas for blend shape target.
        @note
            If no masks are set, default weight value of 1 is used for each delta.
        @param masks
            The weights for each delta.
    */
    DNACAPI void setMasks(ConstArrayView<float> masks);

    /**
        @brief Method for setting the type of operation used to calculate new deltas for blend shape target.
        @note
            Available operations are: Interpolate, Add, Subtract and Multiply. Each delta is calculated based on the provided
       operation type in the following way:

            Interpolate: \f$newValue = previousValue * (1 - weight) + setValue * weight\f$\n
            Add: \f$newValue = previousValue + (setValue * weight)\f$\n
            Subtract: \f$newValue = previousValue - (setValue * weight)\f$\n
            Multiply: \f$newValue = previousValue * (setValue * weight)\f$\n

            setValue is the value from new deltas that were set, and weight is the value from masks array.
        @param operation
            The operation to use.
    */
    DNACAPI void setOperation(VectorOperation operation);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
