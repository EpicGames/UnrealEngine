// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief SetLODsCommand is used to specify LODs to use. Joints, blend shapes, animated maps and meshes that are not in specified
   LODs are removed from the DNA.
*/
class SetLODsCommand : public Command {
public:
    DNACAPI explicit SetLODsCommand(MemoryResource* memRes = nullptr);
    DNACAPI SetLODsCommand(ConstArrayView<std::uint16_t> lods, MemoryResource* memRes = nullptr);

    DNACAPI ~SetLODsCommand();

    SetLODsCommand(const SetLODsCommand&) = delete;
    SetLODsCommand& operator=(const SetLODsCommand&) = delete;

    DNACAPI SetLODsCommand(SetLODsCommand&&);
    DNACAPI SetLODsCommand& operator=(SetLODsCommand&&);

    /**
        @brief Method for setting the LODs to keep.
        @param lods
            New LODs to be used.
    */
    DNACAPI void setLODs(ConstArrayView<std::uint16_t> lods);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
