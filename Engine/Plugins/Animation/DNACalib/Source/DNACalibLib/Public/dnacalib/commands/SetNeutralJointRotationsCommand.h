// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief SetNeutralJointRotationsCommand is used to set new rotation values to neutral joints.
*/
class SetNeutralJointRotationsCommand : public Command {
public:
    DNACAPI explicit SetNeutralJointRotationsCommand(MemoryResource* memRes = nullptr);
    DNACAPI SetNeutralJointRotationsCommand(ConstArrayView<Vector3> rotations, MemoryResource* memRes = nullptr);
    DNACAPI SetNeutralJointRotationsCommand(ConstArrayView<float> xs,
                                            ConstArrayView<float> ys,
                                            ConstArrayView<float> zs,
                                            MemoryResource* memRes = nullptr);

    DNACAPI ~SetNeutralJointRotationsCommand();

    SetNeutralJointRotationsCommand(const SetNeutralJointRotationsCommand&) = delete;
    SetNeutralJointRotationsCommand& operator=(const SetNeutralJointRotationsCommand&) = delete;

    DNACAPI SetNeutralJointRotationsCommand(SetNeutralJointRotationsCommand&&);
    DNACAPI SetNeutralJointRotationsCommand& operator=(SetNeutralJointRotationsCommand&&);

    /**
        @brief Method for setting the neutral joint rotations.
        @param rotations
            Rotation values for each joint.
    */
    DNACAPI void setRotations(ConstArrayView<Vector3> rotations);

    /**
        @brief Method for setting the neutral joint rotations.
        @param xs
            The X rotation value for each joint.
        @param ys
            The Y rotation value for each joint.
        @param zs
            The Z rotation value for each joint.
    */
    DNACAPI void setRotations(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
