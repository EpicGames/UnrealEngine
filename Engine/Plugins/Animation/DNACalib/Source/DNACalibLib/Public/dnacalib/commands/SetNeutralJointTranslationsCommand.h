// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief SetNeutralJointTranslationsCommand is used to set new translation values to neutral joints.
*/
class SetNeutralJointTranslationsCommand : public Command {
public:
    DNACAPI explicit SetNeutralJointTranslationsCommand(MemoryResource* memRes = nullptr);
    DNACAPI SetNeutralJointTranslationsCommand(ConstArrayView<Vector3> translations, MemoryResource* memRes = nullptr);
    DNACAPI SetNeutralJointTranslationsCommand(ConstArrayView<float> xs,
                                               ConstArrayView<float> ys,
                                               ConstArrayView<float> zs,
                                               MemoryResource* memRes = nullptr);

    DNACAPI ~SetNeutralJointTranslationsCommand();

    SetNeutralJointTranslationsCommand(const SetNeutralJointTranslationsCommand&) = delete;
    SetNeutralJointTranslationsCommand& operator=(const SetNeutralJointTranslationsCommand&) = delete;

    DNACAPI SetNeutralJointTranslationsCommand(SetNeutralJointTranslationsCommand&&);
    DNACAPI SetNeutralJointTranslationsCommand& operator=(SetNeutralJointTranslationsCommand&&);

    /**
        @brief Method for setting the neutral joint translations.
        @param translations
            Translation values for each joint.
    */
    DNACAPI void setTranslations(ConstArrayView<Vector3> translations);

    /**
        @brief Method for setting the neutral joint translations.
        @param xs
            The X translation value for each joint.
        @param ys
            The Y translation value for each joint.
        @param zs
            The Z translation value for each joint.
    */
    DNACAPI void setTranslations(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
