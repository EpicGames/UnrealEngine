// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief SetSkinWeightsCommand is used to set new skin weights for a vertex in a mesh.
*/
class SetSkinWeightsCommand : public Command {
public:
    DNACAPI explicit SetSkinWeightsCommand(MemoryResource* memRes = nullptr);
    DNACAPI SetSkinWeightsCommand(std::uint16_t meshIndex,
                                  std::uint32_t vertexIndex,
                                  ConstArrayView<float> weights,
                                  ConstArrayView<std::uint16_t> jointIndices,
                                  MemoryResource* memRes = nullptr);

    DNACAPI ~SetSkinWeightsCommand();

    SetSkinWeightsCommand(const SetSkinWeightsCommand&) = delete;
    SetSkinWeightsCommand& operator=(const SetSkinWeightsCommand&) = delete;

    DNACAPI SetSkinWeightsCommand(SetSkinWeightsCommand&&);
    DNACAPI SetSkinWeightsCommand& operator=(SetSkinWeightsCommand&&);

    /**
        @brief Method for setting the index of the targeted mesh.
        @param meshIndex
            The mesh index.
    */
    DNACAPI void setMeshIndex(std::uint16_t meshIndex);

    /**
        @brief Method for setting the index of the vertex to change.
        @param vertexIndex
            The vertex index.
    */
    DNACAPI void setVertexIndex(std::uint32_t vertexIndex);

    /**
        @brief Method for setting the weights with which joints influence the vertex in question.
        @param weights
            Weights for each joint that has an influence on the vertex.
    */
    DNACAPI void setWeights(ConstArrayView<float> weights);

    /**
        @brief Method for setting the joint indices of joints that influence the vertex in question.
        @param jointIndices
            Joint indices of joints that have an influence on the vertex.
    */
    DNACAPI void setJointIndices(ConstArrayView<std::uint16_t> jointIndices);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
