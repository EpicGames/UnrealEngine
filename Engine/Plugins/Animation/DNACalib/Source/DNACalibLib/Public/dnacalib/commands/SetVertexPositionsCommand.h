// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/commands/VectorOperations.h"
#include "dnacalib/types/Aliases.h"

#include <cstdint>

namespace dnac {

class DNACalibDNAReader;

/**
    @brief SetVertexPositionsCommand is used to change vertex positions values.
*/
class SetVertexPositionsCommand : public Command {
public:
    DNACAPI static const sc::StatusCode PositionsMasksCountMismatch;

public:
    DNACAPI explicit SetVertexPositionsCommand(MemoryResource* memRes = nullptr);

    DNACAPI SetVertexPositionsCommand(std::uint16_t meshIndex,
                                      ConstArrayView<Vector3> positions,
                                      VectorOperation operation,
                                      MemoryResource* memRes = nullptr);
    DNACAPI SetVertexPositionsCommand(std::uint16_t meshIndex,
                                      ConstArrayView<float> xs,
                                      ConstArrayView<float> ys,
                                      ConstArrayView<float> zs,
                                      VectorOperation operation,
                                      MemoryResource* memRes = nullptr);
    DNACAPI SetVertexPositionsCommand(std::uint16_t meshIndex,
                                      ConstArrayView<Vector3> positions,
                                      ConstArrayView<float> masks,
                                      VectorOperation operation,
                                      MemoryResource* memRes = nullptr);
    DNACAPI SetVertexPositionsCommand(std::uint16_t meshIndex,
                                      ConstArrayView<float> xs,
                                      ConstArrayView<float> ys,
                                      ConstArrayView<float> zs,
                                      ConstArrayView<float> masks,
                                      VectorOperation operation,
                                      MemoryResource* memRes = nullptr);

    DNACAPI ~SetVertexPositionsCommand();

    SetVertexPositionsCommand(const SetVertexPositionsCommand&) = delete;
    SetVertexPositionsCommand& operator=(const SetVertexPositionsCommand&) = delete;

    DNACAPI SetVertexPositionsCommand(SetVertexPositionsCommand&&);
    DNACAPI SetVertexPositionsCommand& operator=(SetVertexPositionsCommand&&);

    /**
        @brief Method for setting the index of the mesh to change.
        @param meshIndex
            The mesh index.
    */
    DNACAPI void setMeshIndex(std::uint16_t meshIndex);

    /**
        @brief Method for setting the vertex positions used to calculate new values.
        @param positions
            The vertex positions.
    */
    DNACAPI void setPositions(ConstArrayView<Vector3> positions);

    /**
        @brief Method for setting the vertex positions used to calculate new values.
        @param xs
            The X coordinates for each vertex.
        @param ys
            The Y coordinates for each vertex.
        @param zs
            The Z coordinates for each vertex.
    */
    DNACAPI void setPositions(ConstArrayView<float> xs, ConstArrayView<float> ys, ConstArrayView<float> zs);

    /**
        @brief Method for setting vertex masks used to calculate new vertex position values.
        @note
            If no masks are set, default weight value of 1 is used for each vertex.
        @param masks
            The weights for each vertex.
    */
    DNACAPI void setMasks(ConstArrayView<float> masks);

    /**
        @brief Method for setting the type of operation used to calculate new vertex position values.
        @note
            Available operations are: Interpolate, Add, Subtract and Multiply. Each position is calculated based on the provided
       operation type in the following way:

            Interpolate: \f$newValue = previousValue * (1 - weight) + setValue * weight\f$\n
            Add: \f$newValue = previousValue + (setValue * weight)\f$\n
            Subtract: \f$newValue = previousValue - (setValue * weight)\f$\n
            Multiply: \f$newValue = previousValue * (setValue * weight)\f$\n

            setValue is the value from new positions that were set, and weight is the value from masks array.
        @param operation
            The operation to use.
    */
    DNACAPI void setOperation(VectorOperation operation);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
