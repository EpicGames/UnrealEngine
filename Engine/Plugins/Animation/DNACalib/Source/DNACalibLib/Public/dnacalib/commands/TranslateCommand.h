// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Command.h"
#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

class DNACalibDNAReader;

/**
    @brief TranslateCommand is used to translate neutral joints and vertex positions.
    @note
        Joint translations are represented in parent space, so it is enough to translate only root joints, as that translation
   will be propagated to the rest of the joints.
*/
class TranslateCommand : public Command {
public:
    DNACAPI explicit TranslateCommand(MemoryResource* memRes = nullptr);
    DNACAPI TranslateCommand(Vector3 translation, MemoryResource* memRes = nullptr);

    DNACAPI ~TranslateCommand();

    TranslateCommand(const TranslateCommand&) = delete;
    TranslateCommand& operator=(const TranslateCommand&) = delete;

    DNACAPI TranslateCommand(TranslateCommand&&);
    DNACAPI TranslateCommand& operator=(TranslateCommand&&);

    /**
        @brief Method for setting the translation vector.
        @param translation
            The translation vector.
    */
    DNACAPI void setTranslation(Vector3 translation);
    DNACAPI void run(DNACalibDNAReader* output) override;

private:
    class Impl;
    ScopedPtr<Impl> pImpl;
};

}  // namespace dnac
