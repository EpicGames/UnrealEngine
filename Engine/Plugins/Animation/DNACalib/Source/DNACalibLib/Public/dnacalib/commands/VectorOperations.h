// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace dnac {

enum class VectorOperation {
    Interpolate,
    Add,
    Subtract,
    Multiply
};

}  // namespace dnac
