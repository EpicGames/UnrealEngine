// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

#include <dna/Reader.h>

namespace dnac {

class DNACAPI DNACalibDNAReader : public dna::Reader {
public:
    static DNACalibDNAReader* create(MemoryResource* memRes = nullptr);
    static DNACalibDNAReader* create(const dna::Reader* reader, MemoryResource* memRes = nullptr);
    static void destroy(DNACalibDNAReader* instance);

protected:
    virtual ~DNACalibDNAReader();
};

}  // namespace dnac

namespace pma {

template<>
struct DefaultInstanceCreator<dnac::DNACalibDNAReader> {
    using type = FactoryCreate<dnac::DNACalibDNAReader>;
};

template<>
struct DefaultInstanceDestroyer<dnac::DNACalibDNAReader> {
    using type = FactoryDestroy<dnac::DNACalibDNAReader>;
};

}  // namespace pma
