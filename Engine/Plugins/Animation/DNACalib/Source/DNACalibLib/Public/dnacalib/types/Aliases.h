// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <dna/BinaryStreamReader.h>
#include <dna/BinaryStreamWriter.h>
#include <dna/Configuration.h>
#include <dna/StreamReader.h>
#include <dna/StreamWriter.h>
#include <dna/types/Aliases.h>
#include <dna/types/Vector3.h>
#include <pma/MemoryResource.h>
#include <pma/ScopedPtr.h>
#include <pma/resources/AlignedMemoryResource.h>
#include <pma/resources/ArenaMemoryResource.h>
#include <pma/resources/DefaultMemoryResource.h>
#include <status/Status.h>
#include <status/StatusCode.h>
#include <trio/Stream.h>
#include <trio/streams/FileStream.h>
#include <trio/streams/MemoryMappedFileStream.h>
#include <trio/streams/MemoryStream.h>

namespace dnac {

using dna::BinaryStreamReader;
using dna::BinaryStreamWriter;
using dna::DataLayer;
using dna::RotationUnit;
using dna::StreamReader;
using dna::StreamWriter;
using dna::StringView;
using dna::TranslationUnit;
using dna::UnknownLayerPolicy;
using dna::Vector3;
using sc::Status;
using trio::BoundedIOStream;
using trio::FileStream;
using trio::MemoryMappedFileStream;
using trio::MemoryStream;

template<typename T>
using ArrayView = dna::ArrayView<T>;

template<typename T>
using ConstArrayView = dna::ConstArrayView<T>;

using namespace pma;

}  // namespace dnac
