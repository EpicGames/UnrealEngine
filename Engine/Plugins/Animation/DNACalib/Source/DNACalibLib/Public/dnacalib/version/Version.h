// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define DNAC_MAJOR_VERSION 6
#define DNAC_MINOR_VERSION 16
#define DNAC_PATCH_VERSION 3
#define DNAC_VERSION_STRING "6.16.3"
