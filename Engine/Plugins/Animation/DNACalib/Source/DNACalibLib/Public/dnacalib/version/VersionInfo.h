// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/Defs.h"
#include "dnacalib/types/Aliases.h"

namespace dnac {

struct DNACAPI VersionInfo {
    static int getMajorVersion();
    static int getMinorVersion();
    static int getPatchVersion();
    static StringView getVersionString();
};

}  // namespace dnac
