// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class DNACalibLibTest : ModuleRules
	{
		public DNACalibLibTest(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseUnity = false; // A windows include is preprocessing some method names causing compile failures.

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				PublicDefinitions.Add("GTEST_OS_WINDOWS=1");
			}

			string DNACalibLibPath = Path.GetFullPath(Path.Combine(ModuleDirectory, "../DNACalibLib"));

			if (Target.LinkType == TargetLinkType.Monolithic || Target.bMergeModules)
			{
				PublicDependencyModuleNames.Add("DNACalibLib");
				// Only tests (this module) should have access to Lib module's Private directory, so that's why Private was not made Internal.
				CppCompileWarningSettings.ModuleIncludePrivateWarningLevel = WarningLevel.Off;
				PrivateIncludePaths.Add(Path.Combine(DNACalibLibPath, "Private"));
			}
			else
			{
				PrivateDefinitions.Add("DNACALIB_MODULE_DISCARD");
				ConditionalAddModuleDirectory(new DirectoryReference(DNACalibLibPath));
			}

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"GoogleTest",
					"RigLogicLib"
				}
			);
		}
	}
}
