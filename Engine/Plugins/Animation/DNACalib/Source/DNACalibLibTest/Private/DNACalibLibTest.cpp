// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNACalibLibTest.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FDNACalibLibTest"

DEFINE_LOG_CATEGORY(LogDNACalibLibTest);

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FDNACalibLibTest, DNACalibLibTest)
