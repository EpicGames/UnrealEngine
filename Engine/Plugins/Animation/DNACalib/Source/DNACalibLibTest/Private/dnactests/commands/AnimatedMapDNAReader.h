// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/types/Aliases.h"
#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"

#include <cstdint>

class AnimatedMapDNAReader : public dna::FakeDNACReader {
public:
    explicit AnimatedMapDNAReader(dnac::MemoryResource* memRes = nullptr) :
        animatedMapNames{memRes},
        animatedMapIndicesPerLOD{memRes},
        animatedMapLODs{memRes},
        animatedMapInputIndices{memRes},
        animatedMapOutputIndices{memRes},
        animatedMapFromValues{memRes},
        animatedMapToValues{memRes},
        animatedMapSlopeValues{memRes},
        animatedMapCutValues{memRes} {

        lodCount = 2u;

        animatedMapNames.assign({{"animatedMap1", memRes},
                                 {"animatedMap2", memRes},
                                 {"animatedMap3", memRes},
                                 {"animatedMap4", memRes},
                                 {"animatedMap5", memRes}});
        animatedMapIndicesPerLOD.resize(lodCount);
        animatedMapIndicesPerLOD[0].assign({0u, 1u, 2u, 3u, 4u});
        animatedMapIndicesPerLOD[1].assign({0u, 2u, 3u});
        animatedMapLODs.assign({12u, 8u});
        animatedMapInputIndices.assign({1u, 263u, 21u, 320u, 2u, 20u, 319u, 3u, 21u, 320u, 4u, 5u});
        animatedMapOutputIndices.assign({0u, 0u, 0u, 0u, 1u, 1u, 1u, 2u, 2u, 2u, 3u, 4u});
        animatedMapCutValues.assign({0.0f, 0.0f, 0.0f, -0.066667f, 0.0f, 0.0f, -0.1f, 0.0f, 0.0f, -0.1f, -0.333333f, -0.333333f});
        animatedMapSlopeValues.assign({1.0f, -1.0f, 1.0f, 0.266667f, 1.0f, 0.5f, 0.4f, 1.0f, 0.5f, 0.4f, 1.333333f, 1.333333f});
        animatedMapFromValues.assign({0.0f, 0.0f, 0.0f, 0.25f, 0.0f, 0.0f, 0.25f, 0.0f, 0.0f, 0.25f, 0.25f, 0.25f});
        animatedMapToValues.assign({1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f});
    }

    ~AnimatedMapDNAReader();

    std::uint16_t getLODCount() const override {
        return lodCount;
    }

    std::uint16_t getAnimatedMapCount() const override {
        return static_cast<std::uint16_t>(animatedMapNames.size());
    }

    dnac::StringView getAnimatedMapName(std::uint16_t index) const override {
        return dnac::StringView{animatedMapNames[index]};
    }

    dnac::ConstArrayView<std::uint16_t> getAnimatedMapIndicesForLOD(std::uint16_t lod) const override {
        if (lod < getLODCount()) {
            return dnac::ConstArrayView<std::uint16_t>{animatedMapIndicesPerLOD[lod]};
        }
        return {};
    }

    dnac::ConstArrayView<std::uint16_t> getAnimatedMapLODs() const override {
        return dnac::ConstArrayView<std::uint16_t>{animatedMapLODs};
    }

    dnac::ConstArrayView<std::uint16_t> getAnimatedMapInputIndices() const override {
        return dnac::ConstArrayView<std::uint16_t>{animatedMapInputIndices};
    }

    dnac::ConstArrayView<std::uint16_t> getAnimatedMapOutputIndices() const override {
        return dnac::ConstArrayView<std::uint16_t>{animatedMapOutputIndices};
    }

    dnac::ConstArrayView<float> getAnimatedMapFromValues() const override {
        return dnac::ConstArrayView<float>{animatedMapFromValues};
    }

    dnac::ConstArrayView<float> getAnimatedMapToValues() const override {
        return dnac::ConstArrayView<float>{animatedMapToValues};
    }

    dnac::ConstArrayView<float> getAnimatedMapSlopeValues() const override {
        return dnac::ConstArrayView<float>{animatedMapSlopeValues};
    }

    dnac::ConstArrayView<float> getAnimatedMapCutValues() const override {
        return dnac::ConstArrayView<float>{animatedMapCutValues};
    }

private:
    std::uint16_t lodCount;
    dnac::Vector<dnac::String<char>> animatedMapNames;

    dnac::Matrix<std::uint16_t> animatedMapIndicesPerLOD;
    dnac::Vector<std::uint16_t> animatedMapLODs;
    dnac::Vector<std::uint16_t> animatedMapInputIndices;
    dnac::Vector<std::uint16_t> animatedMapOutputIndices;
    dnac::Vector<float> animatedMapFromValues;
    dnac::Vector<float> animatedMapToValues;
    dnac::Vector<float> animatedMapSlopeValues;
    dnac::Vector<float> animatedMapCutValues;
};
