// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/commands/BlendShapeDNAReader.h"

BlendShapeDNAReader::~BlendShapeDNAReader() = default;
