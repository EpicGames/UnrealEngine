// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/types/Aliases.h"
#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"

#include <cstdint>

class BlendShapeDNAReader : public dna::FakeDNACReader {
public:
    explicit BlendShapeDNAReader(dnac::MemoryResource* memRes = nullptr) :
        blendShapeNames{memRes},
        meshNames{memRes},
        bsChannelIndicesPerLOD{memRes},
        bsChannelLODs{memRes},
        bsChannelInputIndices{memRes},
        bsChannelOutputIndices{memRes},
        bsChannelIndices{memRes},
        bsTargetDeltas{memRes},
        bsTargetVertexIndices{memRes} {

        lodCount = 2u;

        blendShapeNames.assign(
            {{"blendshape1", memRes}, {"blendshape2", memRes}, {"blendshape3", memRes}, {"blendshape4", memRes}});
        meshNames.assign({{"mesh1", memRes}, {"mesh2", memRes}});
        bsChannelIndicesPerLOD.resize(lodCount);
        bsChannelIndicesPerLOD[0].assign({0u, 1u, 2u, 3u});
        bsChannelIndicesPerLOD[1].assign({0u, 2u});
        bsChannelLODs.assign({4u, 2u});
        bsChannelInputIndices.assign({0u, 0u, 1u, 1u});
        bsChannelOutputIndices.assign({1u, 0u, 2u, 3u});

        bsChannelIndices.assign({{0u, 1u, 2u}, {3u}});

        const auto meshCount = BlendShapeDNAReader::getMeshCount();
        bsTargetDeltas.resize(meshCount);
        bsTargetDeltas[0].resize(3u, dnac::RawVector3Vector{memRes});
        float xs1[] = {1.0f, 1.0f, 1.0f, 1.0f, 1.0f};
        float ys1[] = {2.0f, 2.0f, 2.0f, 2.0f, 2.0f};
        float zs1[] = {3.0f, 3.0f, 3.0f, 3.0f, 3.0f};

        float xs2[] = {4.0f, 4.0f, 4.0f, 4.0f, 4.0f};
        float ys2[] = {5.0f, 5.0f, 5.0f, 5.0f, 5.0f};
        float zs2[] = {6.0f, 6.0f, 6.0f, 6.0f, 6.0f};

        float xs3[] = {7.0f, 7.0f, 7.0f, 7.0f, 7.0f};
        float ys3[] = {8.0f, 8.0f, 8.0f, 8.0f, 8.0f};
        float zs3[] = {9.0f, 9.0f, 9.0f, 9.0f, 9.0f};

        float xs4[] = {10.0f, 10.0f, 10.0f};
        float ys4[] = {11.0f, 11.0f, 11.0f};
        float zs4[] = {12.0f, 12.0f, 12.0f};

        bsTargetDeltas[0][0].xs.assign(xs1, xs1 + 5u);
        bsTargetDeltas[0][0].ys.assign(ys1, ys1 + 5u);
        bsTargetDeltas[0][0].zs.assign(zs1, zs1 + 5u);

        bsTargetDeltas[0][1].xs.assign(xs2, xs2 + 5u);
        bsTargetDeltas[0][1].ys.assign(ys2, ys2 + 5u);
        bsTargetDeltas[0][1].zs.assign(zs2, zs2 + 5u);

        bsTargetDeltas[0][2].xs.assign(xs3, xs3 + 5u);
        bsTargetDeltas[0][2].ys.assign(ys3, ys3 + 5u);
        bsTargetDeltas[0][2].zs.assign(zs3, zs3 + 5u);

        bsTargetDeltas[1].resize(1u, dnac::RawVector3Vector{memRes});
        bsTargetDeltas[1][0].xs.assign(xs4, xs4 + 3u);
        bsTargetDeltas[1][0].ys.assign(ys4, ys4 + 3u);
        bsTargetDeltas[1][0].zs.assign(zs4, zs4 + 3u);

        bsTargetVertexIndices.resize(meshCount);
        bsTargetVertexIndices[0].resize(3u);
        bsTargetVertexIndices[0][0].assign({0u, 1u, 2u, 3u, 4u});
        bsTargetVertexIndices[0][1].assign({0u, 1u, 2u, 3u, 4u});
        bsTargetVertexIndices[0][2].assign({0u, 1u, 2u, 3u, 4u});
        bsTargetVertexIndices[1].resize(1u);
        bsTargetVertexIndices[1][0].assign({0u, 1u, 2u});
    }

    ~BlendShapeDNAReader();

    std::uint16_t getLODCount() const override {
        return lodCount;
    }

    std::uint16_t getMeshCount() const override {
        return static_cast<std::uint16_t>(meshNames.size());
    }

    dnac::StringView getMeshName(std::uint16_t index) const override {
        return dnac::StringView{meshNames[index]};
    }

    std::uint16_t getBlendShapeChannelCount() const override {
        return static_cast<std::uint16_t>(blendShapeNames.size());
    }

    dnac::StringView getBlendShapeChannelName(std::uint16_t index) const override {
        return dnac::StringView{blendShapeNames[index]};
    }

    dnac::ConstArrayView<std::uint16_t> getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const override {
        if (lod < getLODCount()) {
            return dnac::ConstArrayView<std::uint16_t>{bsChannelIndicesPerLOD[lod]};
        }
        return {};
    }

    dnac::ConstArrayView<std::uint16_t> getBlendShapeChannelLODs() const override {
        return dnac::ConstArrayView<std::uint16_t>{bsChannelLODs};
    }

    dnac::ConstArrayView<std::uint16_t> getBlendShapeChannelInputIndices() const override {
        return dnac::ConstArrayView<std::uint16_t>{bsChannelInputIndices};
    }

    dnac::ConstArrayView<std::uint16_t> getBlendShapeChannelOutputIndices() const override {
        return dnac::ConstArrayView<std::uint16_t>{bsChannelOutputIndices};
    }

    std::uint16_t getBlendShapeTargetCount(std::uint16_t meshIndex) const override {
        if (meshIndex < getMeshCount()) {
            return static_cast<std::uint16_t>(bsChannelIndices[meshIndex].size());
        }
        return {};
    }

    std::uint16_t getBlendShapeChannelIndex(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return bsChannelIndices[meshIndex][blendShapeTargetIndex];
            }
        }
        return {};
    }

    std::uint32_t getBlendShapeTargetDeltaCount(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return static_cast<std::uint32_t>(bsTargetDeltas[meshIndex][blendShapeTargetIndex].size());
            }
        }
        return {};
    }

    dnac::Vector3 getBlendShapeTargetDelta(std::uint16_t meshIndex,
                                           std::uint16_t blendShapeTargetIndex,
                                           std::uint32_t deltaIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                if (deltaIndex < getBlendShapeTargetDeltaCount(meshIndex, blendShapeTargetIndex)) {
                    return dnac::Vector3{bsTargetDeltas[meshIndex][blendShapeTargetIndex].xs[deltaIndex],
                                         bsTargetDeltas[meshIndex][blendShapeTargetIndex].ys[deltaIndex],
                                         bsTargetDeltas[meshIndex][blendShapeTargetIndex].zs[deltaIndex]};
                }
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].xs};
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].ys};
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].zs};
            }
        }
        return {};
    }

    dnac::ConstArrayView<std::uint32_t> getBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                                         std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<std::uint32_t>{bsTargetVertexIndices[meshIndex][blendShapeTargetIndex]};
            }
        }
        return {};
    }

private:
    std::uint16_t lodCount;
    dnac::Vector<dnac::String<char>> blendShapeNames;
    dnac::Vector<dnac::String<char>> meshNames;

    dnac::Matrix<std::uint16_t> bsChannelIndicesPerLOD;
    dnac::Vector<std::uint16_t> bsChannelLODs;
    dnac::Vector<std::uint16_t> bsChannelInputIndices;
    dnac::Vector<std::uint16_t> bsChannelOutputIndices;

    dnac::Matrix<std::uint16_t> bsChannelIndices;

    dnac::Matrix<dnac::RawVector3Vector> bsTargetDeltas;
    dnac::Matrix<dnac::Vector<std::uint32_t>> bsTargetVertexIndices;
};
