// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/commands/FakeDNACReader.h"

namespace dna {

FakeDNACReader::~FakeDNACReader() = default;

}  // namespace dna
