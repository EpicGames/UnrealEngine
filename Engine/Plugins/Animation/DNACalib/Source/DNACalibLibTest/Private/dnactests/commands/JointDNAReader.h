// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"

#include <cstdint>

class JointDNAReader : public dna::FakeDNACReader {
public:
    explicit JointDNAReader(dnac::MemoryResource* memRes = nullptr) :
        jointNames{memRes},
        jointIndicesPerLOD{memRes},
        jointHierarchy{memRes},
        neutralJointTranslations{memRes},
        neutralJointRotations{memRes},
        jointGroupCount{},
        jointGroupJointIndices{memRes},
        jointGroupLODs{memRes},
        jointGroupInputIndices{memRes},
        jointGroupOutputIndices{memRes},
        jointGroupValues{memRes},
        skinWeightsValues{memRes},
        skinWeightsJointIndices{memRes} {

        lodCount = 2u;

        jointNames.assign({{"JA", memRes}, {"JB", memRes}, {"JC", memRes}, {"JD", memRes}});

        jointIndicesPerLOD.resize(lodCount);
        jointIndicesPerLOD[0].assign({0u, 1u, 2u});
        jointIndicesPerLOD[1].assign({0u, 1u});

        std::uint16_t hierarchy[] = {0, 0, 1, 2};
        jointHierarchy.assign(hierarchy, hierarchy + 4);

        float jxs[] = {1.0f, 4.0f, 7.0f, 10.0f};
        float jys[] = {2.0f, 5.0f, 8.0f, 11.0f};
        float jzs[] = {3.0f, 6.0f, 9.0f, 12.0f};
        neutralJointTranslations.xs.assign(jxs, jxs + 4ul);
        neutralJointTranslations.ys.assign(jys, jys + 4ul);
        neutralJointTranslations.zs.assign(jzs, jzs + 4ul);
        neutralJointRotations.xs.assign(jxs, jxs + 4ul);
        neutralJointRotations.ys.assign(jys, jys + 4ul);
        neutralJointRotations.zs.assign(jzs, jzs + 4ul);

        jointGroupCount = 1u;

        jointGroupJointIndices.assign({0, 1, 2});
        jointGroupLODs.assign({4, 2});
        jointGroupInputIndices.assign({13, 56, 120});
        jointGroupOutputIndices.assign({8, 9, 17, 18});
        jointGroupValues.assign({0.5f, 0.2f, 0.3f, 0.25f, 0.4f, 0.15f, 0.1f, 0.1f, 0.9f, 0.1f, 0.75f, 1.0f});

        skinWeightsJointIndices.resize(4u);
        skinWeightsJointIndices[0].assign({0, 1, 2});
        skinWeightsJointIndices[1].assign({0, 1});
        skinWeightsJointIndices[2].assign({1, 2});
        skinWeightsJointIndices[3].assign({1});

        skinWeightsValues.resize(4u);
        skinWeightsValues[0].assign({0.1f, 0.7f, 0.2f});
        skinWeightsValues[1].assign({0.2f, 0.8f});
        skinWeightsValues[2].assign({0.4f, 0.6f});
        skinWeightsValues[3].assign({1.0f});
    }

    ~JointDNAReader();

    std::uint16_t getLODCount() const override {
        return lodCount;
    }

    std::uint16_t getJointIndexListCount() const override {
        return static_cast<std::uint16_t>(jointIndicesPerLOD.size());
    }

    dnac::ConstArrayView<std::uint16_t> getJointIndicesForLOD(std::uint16_t lod) const override {
        return dnac::ConstArrayView<std::uint16_t>(jointIndicesPerLOD[lod]);
    }

    std::uint16_t getJointParentIndex(std::uint16_t index) const override {
        return jointHierarchy[index];
    }

    std::uint16_t getJointCount() const override {
        return static_cast<std::uint16_t>(jointNames.size());
    }

    dnac::StringView getJointName(std::uint16_t i) const override {
        return dnac::StringView{jointNames[i]};
    }

    std::uint16_t getMeshCount() const override {
        return 1;
    }

    dnac::StringView getMeshName(std::uint16_t /*unused*/) const override {
        return dnac::StringView{"M", 1ul};
    }

    dnac::Vector3 getNeutralJointTranslation(std::uint16_t index) const override {
        return dnac::Vector3{neutralJointTranslations.xs[index],
                             neutralJointTranslations.ys[index],
                             neutralJointTranslations.zs[index]};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationXs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.xs};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationYs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.ys};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationZs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.zs};
    }

    dnac::Vector3 getNeutralJointRotation(std::uint16_t index) const override {
        return dnac::Vector3{neutralJointRotations.xs[index], neutralJointRotations.ys[index], neutralJointRotations.zs[index]};
    }

    dnac::ConstArrayView<float> getNeutralJointRotationXs() const override {
        return dnac::ConstArrayView<float>{neutralJointRotations.xs};
    }

    dnac::ConstArrayView<float> getNeutralJointRotationYs() const override {
        return dnac::ConstArrayView<float>{neutralJointRotations.ys};
    }

    dnac::ConstArrayView<float> getNeutralJointRotationZs() const override {
        return dnac::ConstArrayView<float>{neutralJointRotations.zs};
    }

    std::uint16_t getJointGroupCount() const override {
        return jointGroupCount;
    }

    dnac::ConstArrayView<std::uint16_t> getJointGroupJointIndices(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<std::uint16_t>{jointGroupJointIndices};
    }

    dnac::ConstArrayView<std::uint16_t> getJointGroupLODs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<std::uint16_t>{jointGroupLODs};
    }

    dnac::ConstArrayView<std::uint16_t> getJointGroupInputIndices(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<std::uint16_t>{jointGroupInputIndices};
    }

    dnac::ConstArrayView<std::uint16_t> getJointGroupOutputIndices(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<std::uint16_t>{jointGroupOutputIndices};
    }

    dnac::ConstArrayView<float> getJointGroupValues(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{jointGroupValues};
    }

    std::uint32_t getSkinWeightsCount(std::uint16_t /*unused*/) const override {
        return static_cast<std::uint32_t>(skinWeightsJointIndices.size());
    }

    dnac::ConstArrayView<float> getSkinWeightsValues(std::uint16_t /*unused*/, std::uint32_t vertexIndex) const override {
        return dnac::ConstArrayView<float>{skinWeightsValues[vertexIndex]};
    }

    dnac::ConstArrayView<std::uint16_t> getSkinWeightsJointIndices(std::uint16_t /*unused*/,
                                                                   std::uint32_t vertexIndex) const override {
        return dnac::ConstArrayView<std::uint16_t>{skinWeightsJointIndices[vertexIndex]};
    }

private:
    std::uint16_t lodCount;
    dnac::Vector<dnac::String<char>> jointNames;
    dnac::Matrix<std::uint16_t> jointIndicesPerLOD;
    dnac::Vector<std::uint16_t> jointHierarchy;

    dnac::RawVector3Vector neutralJointTranslations;
    dnac::RawVector3Vector neutralJointRotations;

    std::uint16_t jointGroupCount;
    dnac::Vector<std::uint16_t> jointGroupJointIndices;
    dnac::Vector<std::uint16_t> jointGroupLODs;
    dnac::Vector<std::uint16_t> jointGroupInputIndices;
    dnac::Vector<std::uint16_t> jointGroupOutputIndices;
    dnac::Vector<float> jointGroupValues;

    dnac::Matrix<float> skinWeightsValues;
    dnac::Matrix<std::uint16_t> skinWeightsJointIndices;
};
