// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnacalib/types/Aliases.h"
#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"

#include <dna/Reader.h>

#include <cstdint>

class MeshDNAReader : public dna::FakeDNACReader {
public:
    explicit MeshDNAReader(dnac::MemoryResource* memRes = nullptr) :
        blendShapeNames{memRes},
        blendShapeChannelIndicesPerLOD{memRes},
        meshNames{memRes},
        meshBlendShapeChannelMappings{memRes},
        meshBlendShapeChannelMappingIndicesPerLOD{memRes},
        meshIndicesPerLOD{memRes},
        vertexPositions{memRes} {

        lodCount = 2u;

        meshNames.assign({{"mesh0", memRes}, {"mesh1", memRes}, {"mesh2", memRes}});
        blendShapeNames.assign({{"blendshape0", memRes},
                                {"blendshape1", memRes},
                                {"blendshape2", memRes},
                                {"blendshape3", memRes},
                                {"blendshape4", memRes},
                                {"blendshape5", memRes}});
        blendShapeChannelIndicesPerLOD.resize(lodCount);
        blendShapeChannelIndicesPerLOD[0].assign({0u, 1u, 2u, 3u, 4u, 5u});
        blendShapeChannelIndicesPerLOD[1].assign({2u, 3u, 4u, 5u});

        meshBlendShapeChannelMappings = {
            dna::MeshBlendShapeChannelMapping{0, 0},  // mesh0 - blendshape0
            dna::MeshBlendShapeChannelMapping{0, 1},  // mesh0 - blendshape1
            dna::MeshBlendShapeChannelMapping{1, 2},  // mesh1 - blendshape2
            dna::MeshBlendShapeChannelMapping{1, 3},  // mesh1 - blendshape3
            dna::MeshBlendShapeChannelMapping{2, 4},  // mesh2 - blendshape4
            dna::MeshBlendShapeChannelMapping{2, 5}   // mesh2 - blendshape5
        };
        meshBlendShapeChannelMappingIndicesPerLOD = {
            {0, 1, 2, 3, 4, 5},  // lod-0
            {2, 3, 4, 5}         // lod-1
        };
        meshIndicesPerLOD = {
            {0, 1, 2},  // lod-0
            {1, 2}      // lod-1
        };
        vertexPositions = {{
                               // mesh1
                               {0.0f, 0.0f, 0.0f},  // Xs
                               {1.0f, 1.0f, 1.0f},  // Ys
                               {2.0f, 2.0f, 2.0f}   // Zs
                           },
                           {
                               // mesh2
                               {3.0f, 3.0f},  // Xs
                               {4.0f, 4.0f},  // Ys
                               {5.0f, 5.0f}   // Zs
                           },
                           {
                               // mesh3
                               {6.0f, 6.0f},  // Xs
                               {7.0f, 7.0f},  // Ys
                               {8.0f, 8.0f}   // Zs
                           }};
    }

    ~MeshDNAReader();

    std::uint16_t getLODCount() const override {
        return lodCount;
    }

    std::uint16_t getMeshCount() const override {
        return static_cast<std::uint16_t>(meshNames.size());
    }

    dnac::StringView getMeshName(std::uint16_t index) const override {
        return dnac::StringView{meshNames[index]};
    }

    std::uint16_t getBlendShapeChannelCount() const override {
        return static_cast<std::uint16_t>(blendShapeNames.size());
    }

    dnac::StringView getBlendShapeChannelName(std::uint16_t index) const override {
        return dnac::StringView{blendShapeNames[index]};
    }

    dnac::ConstArrayView<std::uint16_t> getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const override {
        return dnac::ConstArrayView<std::uint16_t>{blendShapeChannelIndicesPerLOD[lod]};
    }

    std::uint16_t getMeshBlendShapeChannelMappingCount() const override {
        return static_cast<std::uint16_t>(meshBlendShapeChannelMappings.size());
    }

    dna::MeshBlendShapeChannelMapping getMeshBlendShapeChannelMapping(std::uint16_t index) const override {
        return meshBlendShapeChannelMappings[index];
    }

    dnac::ConstArrayView<std::uint16_t> getMeshBlendShapeChannelMappingIndicesForLOD(std::uint16_t lod) const override {
        return dnac::ConstArrayView<std::uint16_t>{meshBlendShapeChannelMappingIndicesPerLOD[lod]};
    }

    std::uint16_t getMeshIndexListCount() const override {
        return static_cast<std::uint16_t>(meshIndicesPerLOD.size());
    }

    dnac::ConstArrayView<std::uint16_t> getMeshIndicesForLOD(std::uint16_t lod) const override {
        return dnac::ConstArrayView<std::uint16_t>{meshIndicesPerLOD[lod]};
    }

    std::uint32_t getVertexPositionCount(std::uint16_t meshIndex) const override {
        return static_cast<std::uint32_t>(vertexPositions[meshIndex][0].size());
    }

    dna::Position getVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override {
        return dna::Position{vertexPositions[meshIndex][0][vertexIndex],
                             vertexPositions[meshIndex][1][vertexIndex],
                             vertexPositions[meshIndex][2][vertexIndex]};
    }

    dnac::ConstArrayView<float> getVertexPositionXs(std::uint16_t meshIndex) const override {
        return dnac::ConstArrayView<float>{vertexPositions[meshIndex][0]};
    }

    dnac::ConstArrayView<float> getVertexPositionYs(std::uint16_t meshIndex) const override {
        return dnac::ConstArrayView<float>{vertexPositions[meshIndex][1]};
    }

    dnac::ConstArrayView<float> getVertexPositionZs(std::uint16_t meshIndex) const override {
        return dnac::ConstArrayView<float>{vertexPositions[meshIndex][2]};
    }

private:
    std::uint16_t lodCount;
    dnac::Vector<dnac::String<char>> blendShapeNames;
    dnac::Matrix<std::uint16_t> blendShapeChannelIndicesPerLOD;
    dnac::Vector<dnac::String<char>> meshNames;
    dnac::Vector<dna::MeshBlendShapeChannelMapping> meshBlendShapeChannelMappings;
    dnac::Matrix<std::uint16_t> meshBlendShapeChannelMappingIndicesPerLOD;
    dnac::Matrix<std::uint16_t> meshIndicesPerLOD;
    dnac::Matrix<dnac::Vector<float>> vertexPositions;
};
