// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"

#include "dnacalib/commands/CalculateMeshLowerLODsCommandImpl.h"

TEST(TestCalculateMeshLowerLODsCommandImpl, IsUVMapOverlapping) {
    std::vector<float> us{0.5f, 0.2f, 0.7f, 0.5f, 0.7f, 0.2f};
    std::vector<float> vs{0.3f, 0.1f, 0.4f, 0.3f, 0.4f, 0.1f};
    ASSERT_TRUE(dnac::isUVMapOverlapping(us, vs, 3ul));
}

TEST(TestCalculateMeshLowerLODsCommandImpl, IsUVMapOverlappingUnevenSize) {
    ASSERT_FALSE(dnac::isUVMapOverlapping({nullptr, 1ul}, {nullptr, 1ul}));
}

TEST(TestCalculateMeshLowerLODsCommandImpl, IsUVMapOverlappingInsufficientOverlaps) {
    std::vector<float> us{0.5f, 0.2f, 0.5f, 0.8f};
    std::vector<float> vs{0.5f, 0.2f, 0.5f, 0.8f};
    ASSERT_FALSE(dnac::isUVMapOverlapping(us, vs, 2ul));
}

TEST(TestCalculateMeshLowerLODsCommandImpl, OffsetOverlappingUVMapRegionUsOnly) {
    std::vector<float> us{0.5f, 0.2f, 0.7f, 0.5f, 0.7f, 0.2f};
    std::vector<float> vs{0.3f, 0.1f, 0.4f, 0.3f, 0.4f, 0.1f};
    dnac::offsetOverlappingUVMapRegion(us, vs, 1.0f, 0.0f);

    std::vector<float> usExpected{1.5f, 1.2f, 1.7f, 0.5f, 0.7f, 0.2f};
    std::vector<float> vsExpected{0.3f, 0.1f, 0.4f, 0.3f, 0.4f, 0.1f};
    ASSERT_ELEMENTS_EQ(us, usExpected, us.size());
    ASSERT_ELEMENTS_EQ(vs, vsExpected, vs.size());
}

TEST(TestCalculateMeshLowerLODsCommandImpl, OffsetOverlappingUVMapRegionVsOnly) {
    std::vector<float> us{0.5f, 0.2f, 0.7f, 0.5f, 0.7f, 0.2f};
    std::vector<float> vs{0.3f, 0.1f, 0.4f, 0.3f, 0.4f, 0.1f};
    dnac::offsetOverlappingUVMapRegion(us, vs, 0.0f, 1.0f);

    std::vector<float> usExpected{0.5f, 0.2f, 0.7f, 0.5f, 0.7f, 0.2f};
    std::vector<float> vsExpected{1.3f, 1.1f, 1.4f, 0.3f, 0.4f, 0.1f};
    ASSERT_ELEMENTS_EQ(us, usExpected, us.size());
    ASSERT_ELEMENTS_EQ(vs, vsExpected, vs.size());
}
