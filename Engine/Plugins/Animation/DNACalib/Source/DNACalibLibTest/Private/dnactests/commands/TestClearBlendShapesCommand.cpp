// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"

#include <cstdint>

namespace {

class DNAReader : public dna::FakeDNACReader {
public:
    explicit DNAReader(dnac::MemoryResource* memRes = nullptr) :
        meshNames{memRes},
        blendShapeTargetVertexIndices{memRes},
        blendShapeTargetDeltas{memRes},
        blendShapeChannelLODs{memRes},
        blendShapeChannelInputIndices{memRes},
        blendShapeChannelOutputIndices{memRes} {

        lodCount = 2u;
        meshNames = {"mesh_1", "mesh_2", "mesh_3"};

        const auto meshCount = static_cast<std::uint16_t>(meshNames.size());
        blendShapeTargetVertexIndices.resize(meshCount);
        blendShapeTargetVertexIndices[0].resize(2u);
        blendShapeTargetVertexIndices[1].resize(1u);
        blendShapeTargetVertexIndices[2].resize(1u);

        std::uint32_t vtxIndices1[] = {0u, 1u, 2u, 3u, 4u, 5u, 6u};
        std::uint32_t vtxIndices2[] = {0u, 2u, 6u};

        blendShapeTargetVertexIndices[0][0].assign(vtxIndices1, vtxIndices1 + 7ul);
        blendShapeTargetVertexIndices[0][1].assign(vtxIndices1, vtxIndices1 + 7ul);
        blendShapeTargetVertexIndices[1][0].assign(vtxIndices1, vtxIndices1 + 7ul);
        blendShapeTargetVertexIndices[2][0].assign(vtxIndices2, vtxIndices2 + 3ul);

        blendShapeTargetDeltas.resize(meshCount);
        blendShapeTargetDeltas[0].resize(2u, dnac::RawVector3Vector{memRes});
        blendShapeTargetDeltas[1].resize(1u, dnac::RawVector3Vector{memRes});
        blendShapeTargetDeltas[2].resize(1u, dnac::RawVector3Vector{memRes});

        float bxs1[] = {0.0005f, 0.0015f, 0.002f, 0.005f, 0.01f, 0.001f, 0.1f};
        float bys1[] = {0.0005f, 0.0015f, 0.002f, 0.005f, 0.01f, 0.001f, 0.1f};
        float bzs1[] = {0.0005f, 0.0015f, 0.002f, 0.005f, 0.01f, 0.001f, 0.1f};
        float bxs2[] = {0.002f, 0.01f, 0.1f};
        float bys2[] = {0.002f, 0.01f, 0.1f};
        float bzs2[] = {0.002f, 0.01f, 0.1f};

        blendShapeTargetDeltas[0][0].xs.assign(bxs1, bxs1 + 7ul);
        blendShapeTargetDeltas[0][0].ys.assign(bys1, bys1 + 7ul);
        blendShapeTargetDeltas[0][0].zs.assign(bzs1, bzs1 + 7ul);
        blendShapeTargetDeltas[0][1].xs.assign(bxs1, bxs1 + 7ul);
        blendShapeTargetDeltas[0][1].ys.assign(bys1, bys1 + 7ul);
        blendShapeTargetDeltas[0][1].zs.assign(bzs1, bzs1 + 7ul);

        blendShapeTargetDeltas[1][0].xs.assign(bxs1, bxs1 + 7ul);
        blendShapeTargetDeltas[1][0].ys.assign(bys1, bys1 + 7ul);
        blendShapeTargetDeltas[1][0].zs.assign(bzs1, bzs1 + 7ul);

        blendShapeTargetDeltas[2][0].xs.assign(bxs2, bxs2 + 3ul);
        blendShapeTargetDeltas[2][0].ys.assign(bys2, bys2 + 3ul);
        blendShapeTargetDeltas[2][0].zs.assign(bzs2, bzs2 + 3ul);

        blendShapeChannelLODs = {3, 1};
        blendShapeChannelInputIndices = {3, 7, 9};
        blendShapeChannelOutputIndices = {1, 3, 5, 6};
    }

    std::uint16_t getLODCount() const override {
        return lodCount;
    }

    std::uint16_t getMeshCount() const override {
        return static_cast<std::uint16_t>(meshNames.size());
    }

    dnac::StringView getMeshName(std::uint16_t meshIndex) const override {
        if (meshIndex < getMeshCount()) {
            return dnac::StringView{meshNames[meshIndex].data(), meshNames[meshIndex].size()};
        }
        return {};
    }

    std::uint16_t getBlendShapeTargetCount(std::uint16_t meshIndex) const override {
        if (meshIndex < getMeshCount()) {
            return static_cast<std::uint16_t>(blendShapeTargetVertexIndices[meshIndex].size());
        }
        return {};
    }

    std::uint32_t getBlendShapeTargetDeltaCount(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
            return static_cast<std::uint16_t>(blendShapeTargetVertexIndices[meshIndex][blendShapeTargetIndex].size());
        }
        return {};
    }

    dnac::Vector3 getBlendShapeTargetDelta(std::uint16_t meshIndex,
                                           std::uint16_t blendShapeTargetIndex,
                                           std::uint32_t deltaIndex) const override {
        if (deltaIndex < getBlendShapeTargetDeltaCount(meshIndex, blendShapeTargetIndex)) {
            return dnac::Vector3{blendShapeTargetDeltas[meshIndex][blendShapeTargetIndex].xs[deltaIndex],
                                 blendShapeTargetDeltas[meshIndex][blendShapeTargetIndex].ys[deltaIndex],
                                 blendShapeTargetDeltas[meshIndex][blendShapeTargetIndex].zs[deltaIndex]};
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
            return dnac::ConstArrayView<float>{blendShapeTargetDeltas[meshIndex][blendShapeTargetIndex].xs};
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
            return dnac::ConstArrayView<float>{blendShapeTargetDeltas[meshIndex][blendShapeTargetIndex].ys};
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
            return dnac::ConstArrayView<float>{blendShapeTargetDeltas[meshIndex][blendShapeTargetIndex].zs};
        }
        return {};
    }

    dnac::ConstArrayView<std::uint32_t> getBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                                         std::uint16_t blendShapeTargetIndex) const override {
        if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
            return dnac::ConstArrayView<std::uint32_t>{blendShapeTargetVertexIndices[meshIndex][blendShapeTargetIndex]};
        }
        return {};
    }

    dnac::ConstArrayView<std::uint16_t> getBlendShapeChannelLODs() const override {
        return dnac::ConstArrayView<std::uint16_t>{blendShapeChannelLODs};
    }

    dnac::ConstArrayView<std::uint16_t> getBlendShapeChannelInputIndices() const override {
        return dnac::ConstArrayView<std::uint16_t>{blendShapeChannelInputIndices};
    }

    dnac::ConstArrayView<std::uint16_t> getBlendShapeChannelOutputIndices() const override {
        return dnac::ConstArrayView<std::uint16_t>{blendShapeChannelOutputIndices};
    }

private:
    std::uint16_t lodCount;
    dnac::Vector<dnac::String<char>> meshNames;
    dnac::Matrix<dnac::Vector<std::uint32_t>> blendShapeTargetVertexIndices;
    dnac::Matrix<dnac::RawVector3Vector> blendShapeTargetDeltas;
    dnac::Vector<std::uint16_t> blendShapeChannelLODs;
    dnac::Vector<std::uint16_t> blendShapeChannelInputIndices;
    dnac::Vector<std::uint16_t> blendShapeChannelOutputIndices;
};

class ClearBlendShapesCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        DNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;
    dnac::DefaultMemoryResource memRes;
};

}  // namespace

TEST_F(ClearBlendShapesCommandTest, ClearAllBlendShapes) {
    dnac::ClearBlendShapesCommand clearBSCmd{&memRes};
    ASSERT_EQ(output->getMeshCount(), 3u);
    ASSERT_EQ(output->getBlendShapeTargetCount(0), 2u);
    ASSERT_EQ(output->getBlendShapeTargetCount(1), 1u);
    ASSERT_EQ(output->getBlendShapeTargetCount(2), 1u);
    ASSERT_EQ(output->getBlendShapeTargetDeltaCount(0, 0), 7u);
    ASSERT_EQ(output->getBlendShapeTargetDeltaCount(0, 1), 7u);
    ASSERT_EQ(output->getBlendShapeTargetDeltaCount(1, 0), 7u);
    ASSERT_EQ(output->getBlendShapeTargetDeltaCount(2, 0), 3u);
    ASSERT_EQ(output->getBlendShapeChannelLODs().size(), 2u);
    ASSERT_EQ(output->getBlendShapeChannelInputIndices().size(), 3u);
    ASSERT_EQ(output->getBlendShapeChannelOutputIndices().size(), 4u);
    clearBSCmd.run(output.get());
    ASSERT_EQ(output->getMeshCount(), 3u);
    ASSERT_EQ(output->getBlendShapeTargetCount(0), 0u);
    ASSERT_EQ(output->getBlendShapeTargetCount(1), 0u);
    ASSERT_EQ(output->getBlendShapeTargetCount(2), 0u);
    ASSERT_EQ(output->getBlendShapeTargetDeltaCount(0, 0), 0u);
    ASSERT_EQ(output->getBlendShapeTargetDeltaCount(0, 1), 0u);
    ASSERT_EQ(output->getBlendShapeTargetDeltaCount(1, 0), 0u);
    ASSERT_EQ(output->getBlendShapeTargetDeltaCount(2, 0), 0u);
    ASSERT_EQ(output->getBlendShapeChannelLODs().size(), 2u);
    std::vector<std::uint16_t> expectedLODs = {0u, 0u};
    ASSERT_ELEMENTS_EQ(output->getBlendShapeChannelLODs(), expectedLODs, expectedLODs.size());
    ASSERT_EQ(output->getBlendShapeChannelInputIndices().size(), 0u);
    ASSERT_EQ(output->getBlendShapeChannelOutputIndices().size(), 0u);
}
