// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"

#include <array>
#include <cstdint>

namespace {

class ConvertibleDNAReader : public dna::FakeDNACReader {
public:
    ConvertibleDNAReader(dna::TranslationUnit translationUnit_,
                         dna::RotationUnit rotationUnit_,
                         dnac::MemoryResource* memRes = nullptr) :
        translationUnit{translationUnit_},
        rotationUnit{rotationUnit_},
        neutralJointRotations{memRes},
        neutralJointTranslations{memRes},
        vertexPositions{memRes},
        blendShapeNames{memRes},
        bsChannelIndices{memRes},
        bsTargetDeltas{memRes},
        bsTargetVertexIndices{memRes},
        jointGroupCount{},
        jointGroupJointIndices{memRes},
        jointGroupLODs{memRes},
        jointGroupInputIndices{memRes},
        jointGroupOutputIndices{memRes},
        jointGroupValues{memRes} {

        jointHierarchy = {0, 0};
        float jrxs[] = {1.0f, 2.5f};
        float jrys[] = {3.0f, 4.5f};
        float jrzs[] = {4.0f, 8.0f};
        neutralJointRotations.xs.assign(jrxs, jrxs + 2ul);
        neutralJointRotations.ys.assign(jrys, jrys + 2ul);
        neutralJointRotations.zs.assign(jrzs, jrzs + 2ul);

        float jtxs[] = {1.0f, 2.5f};
        float jtys[] = {3.0f, 4.5f};
        float jtzs[] = {4.0f, 8.0f};
        neutralJointTranslations.xs.assign(jtxs, jtxs + 2ul);
        neutralJointTranslations.ys.assign(jtys, jtys + 2ul);
        neutralJointTranslations.zs.assign(jtzs, jtzs + 2ul);

        float vxs[] = {4.0f, 12.0f, 23.5f, -4.0f, 2.0f};
        float vys[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f};
        float vzs[] = {11.0f, -5.5f, 22.0f, 3.0f, 6.1f};
        vertexPositions.xs.assign(vxs, vxs + 5ul);
        vertexPositions.ys.assign(vys, vys + 5ul);
        vertexPositions.zs.assign(vzs, vzs + 5ul);

        bsChannelIndices.assign({{0u, 1u, 2u}});

        const auto meshCount = ConvertibleDNAReader::getMeshCount();
        bsTargetDeltas.resize(meshCount);
        bsTargetDeltas[0].resize(3u, dnac::RawVector3Vector{memRes});
        float xs1[] = {1.0f, 1.0f, 1.0f, 1.0f, 1.0f};
        float ys1[] = {2.0f, 2.0f, 2.0f, 2.0f, 2.0f};
        float zs1[] = {3.0f, 3.0f, 3.0f, 3.0f, 3.0f};

        float xs2[] = {4.0f, 4.0f, 4.0f, 4.0f, 4.0f};
        float ys2[] = {5.0f, 5.0f, 5.0f, 5.0f, 5.0f};
        float zs2[] = {6.0f, 6.0f, 6.0f, 6.0f, 6.0f};

        float xs3[] = {7.0f, 7.0f, 7.0f, 7.0f, 7.0f};
        float ys3[] = {8.0f, 8.0f, 8.0f, 8.0f, 8.0f};
        float zs3[] = {9.0f, 9.0f, 9.0f, 9.0f, 9.0f};

        bsTargetDeltas[0][0].xs.assign(xs1, xs1 + 5u);
        bsTargetDeltas[0][0].ys.assign(ys1, ys1 + 5u);
        bsTargetDeltas[0][0].zs.assign(zs1, zs1 + 5u);

        bsTargetDeltas[0][1].xs.assign(xs2, xs2 + 5u);
        bsTargetDeltas[0][1].ys.assign(ys2, ys2 + 5u);
        bsTargetDeltas[0][1].zs.assign(zs2, zs2 + 5u);

        bsTargetDeltas[0][2].xs.assign(xs3, xs3 + 5u);
        bsTargetDeltas[0][2].ys.assign(ys3, ys3 + 5u);
        bsTargetDeltas[0][2].zs.assign(zs3, zs3 + 5u);

        bsTargetVertexIndices.resize(meshCount);
        bsTargetVertexIndices[0].resize(3u);
        bsTargetVertexIndices[0][0].assign({0u, 1u, 2u, 3u, 4u});
        bsTargetVertexIndices[0][1].assign({0u, 1u, 2u, 3u, 4u});
        bsTargetVertexIndices[0][2].assign({0u, 1u, 2u, 3u, 4u});

        jointGroupCount = 1u;

        jointGroupJointIndices.assign({0, 1, 2});
        jointGroupLODs.assign({5, 2});
        jointGroupInputIndices.assign({13, 56, 120});
        jointGroupOutputIndices.assign({9, 11, 12, 14, 15});
        jointGroupValues.assign({0.5f, 0.2f, 0.3f, 0.25f, 0.4f, 0.15f, 0.1f, 0.1f, 0.9f, 0.1f, 0.75f, 1.0f, 0.3f, 0.7f, 0.45f});
    }

    dna::TranslationUnit getTranslationUnit() const override {
        return translationUnit;
    }

    dna::RotationUnit getRotationUnit() const override {
        return rotationUnit;
    }

    std::uint16_t getJointParentIndex(std::uint16_t index) const override {
        return jointHierarchy[index];
    }

    std::uint16_t getJointCount() const override {
        return static_cast<std::uint16_t>(jointHierarchy.size());
    }

    dnac::StringView getJointName(std::uint16_t /*unused*/) const override {
        return dnac::StringView{"A", 1ul};
    }

    std::uint16_t getMeshCount() const override {
        return 1;
    }

    dnac::StringView getMeshName(std::uint16_t /*unused*/) const override {
        return dnac::StringView{"M", 1ul};
    }

    dnac::Vector3 getNeutralJointRotation(std::uint16_t index) const override {
        return dnac::Vector3{neutralJointRotations.xs[index], neutralJointRotations.ys[index], neutralJointRotations.zs[index]};
    }

    dnac::ConstArrayView<float> getNeutralJointRotationXs() const override {
        return dnac::ConstArrayView<float>{neutralJointRotations.xs};
    }

    dnac::ConstArrayView<float> getNeutralJointRotationYs() const override {
        return dnac::ConstArrayView<float>{neutralJointRotations.ys};
    }

    dnac::ConstArrayView<float> getNeutralJointRotationZs() const override {
        return dnac::ConstArrayView<float>{neutralJointRotations.zs};
    }

    dnac::Vector3 getNeutralJointTranslation(std::uint16_t index) const override {
        return dnac::Vector3{neutralJointTranslations.xs[index],
                             neutralJointTranslations.ys[index],
                             neutralJointTranslations.zs[index]};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationXs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.xs};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationYs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.ys};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationZs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.zs};
    }

    std::uint16_t getJointGroupCount() const override {
        return jointGroupCount;
    }

    dnac::ConstArrayView<std::uint16_t> getJointGroupJointIndices(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<std::uint16_t>{jointGroupJointIndices};
    }

    dnac::ConstArrayView<std::uint16_t> getJointGroupLODs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<std::uint16_t>{jointGroupLODs};
    }

    dnac::ConstArrayView<std::uint16_t> getJointGroupInputIndices(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<std::uint16_t>{jointGroupInputIndices};
    }

    dnac::ConstArrayView<std::uint16_t> getJointGroupOutputIndices(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<std::uint16_t>{jointGroupOutputIndices};
    }

    dnac::ConstArrayView<float> getJointGroupValues(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{jointGroupValues};
    }

    std::uint32_t getVertexPositionCount(std::uint16_t /*unused*/) const override {
        return static_cast<std::uint32_t>(vertexPositions.size());
    }

    dnac::Vector3 getVertexPosition(std::uint16_t /*unused*/, std::uint32_t vertexIndex) const override {
        return dnac::Vector3{vertexPositions.xs[vertexIndex], vertexPositions.ys[vertexIndex], vertexPositions.zs[vertexIndex]};
    }

    dnac::ConstArrayView<float> getVertexPositionXs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.xs};
    }

    dnac::ConstArrayView<float> getVertexPositionYs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.ys};
    }

    dnac::ConstArrayView<float> getVertexPositionZs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.zs};
    }

    std::uint16_t getBlendShapeChannelCount() const override {
        return static_cast<std::uint16_t>(blendShapeNames.size());
    }

    dnac::StringView getBlendShapeChannelName(std::uint16_t index) const override {
        return dnac::StringView{blendShapeNames[index]};
    }

    std::uint16_t getBlendShapeTargetCount(std::uint16_t meshIndex) const override {
        if (meshIndex < getMeshCount()) {
            return static_cast<std::uint16_t>(bsChannelIndices[meshIndex].size());
        }
        return {};
    }

    std::uint16_t getBlendShapeChannelIndex(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return bsChannelIndices[meshIndex][blendShapeTargetIndex];
            }
        }
        return {};
    }

    std::uint32_t getBlendShapeTargetDeltaCount(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return static_cast<std::uint32_t>(bsTargetDeltas[meshIndex][blendShapeTargetIndex].size());
            }
        }
        return {};
    }

    dnac::Vector3 getBlendShapeTargetDelta(std::uint16_t meshIndex,
                                           std::uint16_t blendShapeTargetIndex,
                                           std::uint32_t deltaIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                if (deltaIndex < getBlendShapeTargetDeltaCount(meshIndex, blendShapeTargetIndex)) {
                    return dnac::Vector3{bsTargetDeltas[meshIndex][blendShapeTargetIndex].xs[deltaIndex],
                                         bsTargetDeltas[meshIndex][blendShapeTargetIndex].ys[deltaIndex],
                                         bsTargetDeltas[meshIndex][blendShapeTargetIndex].zs[deltaIndex]};
                }
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].xs};
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].ys};
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].zs};
            }
        }
        return {};
    }

    dnac::ConstArrayView<std::uint32_t> getBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                                         std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<std::uint32_t>{bsTargetVertexIndices[meshIndex][blendShapeTargetIndex]};
            }
        }
        return {};
    }

private:
    dnac::TranslationUnit translationUnit;
    dnac::RotationUnit rotationUnit;

    std::array<std::uint16_t, 2> jointHierarchy;
    dnac::RawVector3Vector neutralJointRotations;
    dnac::RawVector3Vector neutralJointTranslations;

    dnac::RawVector3Vector vertexPositions;
    dnac::Vector<dnac::String<char>> blendShapeNames;
    dnac::Matrix<std::uint16_t> bsChannelIndices;
    dnac::Matrix<dnac::RawVector3Vector> bsTargetDeltas;
    dnac::Matrix<dnac::Vector<std::uint32_t>> bsTargetVertexIndices;

    std::uint16_t jointGroupCount;
    dnac::Vector<std::uint16_t> jointGroupJointIndices;
    dnac::Vector<std::uint16_t> jointGroupLODs;
    dnac::Vector<std::uint16_t> jointGroupInputIndices;
    dnac::Vector<std::uint16_t> jointGroupOutputIndices;
    dnac::Vector<float> jointGroupValues;
};

}  // namespace

TEST(ConvertUnitsCommandTest, DegreesToRadiansCMToM) {
    ConvertibleDNAReader fixtures{dnac::TranslationUnit::cm, dnac::RotationUnit::degrees};
    auto output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);

    ASSERT_EQ(output->getTranslationUnit(), dnac::TranslationUnit::cm);
    ASSERT_EQ(output->getRotationUnit(), dnac::RotationUnit::degrees);

    dnac::ConvertUnitsCommand cmd(dnac::TranslationUnit::m, dnac::RotationUnit::radians);
    cmd.run(output.get());

    ASSERT_EQ(output->getTranslationUnit(), dnac::TranslationUnit::m);
    ASSERT_EQ(output->getRotationUnit(), dnac::RotationUnit::radians);

    float jtxs[] = {0.01f, 0.025f};
    float jtys[] = {0.03f, 0.045f};
    float jtzs[] = {0.04f, 0.08f};
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationXs(), jtxs, 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationYs(), jtys, 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationZs(), jtzs, 2ul, 0.0001f);

    float jrxs[] = {0.0174533f, 0.04363325f};
    float jrys[] = {0.0523599f, 0.07853985f};
    float jrzs[] = {0.0698132f, 0.1396264f};
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointRotationXs(), jrxs, 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointRotationYs(), jrys, 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointRotationZs(), jrzs, 2ul, 0.0001f);

    float jgvs[] = {0.005f,
                    0.002f,
                    0.003f,
                    0.0025f,
                    0.004f,
                    0.0015f,
                    0.00174533f,
                    0.00174533f,
                    0.01570797f,
                    0.00174533f,
                    0.013089975f,
                    0.0174533f,
                    0.3f,
                    0.7f,
                    0.45f};
    ASSERT_ELEMENTS_NEAR(output->getJointGroupValues(0), jgvs, 15ul, 0.0001f);

    const std::size_t vertexCount = 5ul;

    float vxs[] = {0.04f, 0.12f, 0.235f, -0.04f, 0.02f};
    float vys[] = {0.01f, 0.02f, 0.03f, 0.04f, 0.05f};
    float vzs[] = {0.11f, -0.055f, 0.22f, 0.03f, 0.061f};
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionXs(0), vxs, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionYs(0), vys, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionZs(0), vzs, vertexCount, 0.0001f);

    float bsxs1[] = {0.01f, 0.01f, 0.01f, 0.01f, 0.01f};
    float bsys1[] = {0.02f, 0.02f, 0.02f, 0.02f, 0.02f};
    float bszs1[] = {0.03f, 0.03f, 0.03f, 0.03f, 0.03f};

    float bsxs2[] = {0.04f, 0.04f, 0.04f, 0.04f, 0.04f};
    float bsys2[] = {0.05f, 0.05f, 0.05f, 0.05f, 0.05f};
    float bszs2[] = {0.06f, 0.06f, 0.06f, 0.06f, 0.06f};

    float bsxs3[] = {0.07f, 0.07f, 0.07f, 0.07f, 0.07f};
    float bsys3[] = {0.08f, 0.08f, 0.08f, 0.08f, 0.08f};
    float bszs3[] = {0.09f, 0.09f, 0.09f, 0.09f, 0.09f};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(0, 0).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(0, 0).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(0, 0).size(), vertexCount);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(0, 0), bsxs1, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(0, 0), bsys1, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(0, 0), bszs1, vertexCount, 0.0001f);
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(0, 1).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(0, 1).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(0, 1).size(), vertexCount);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(0, 1), bsxs2, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(0, 1), bsys2, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(0, 1), bszs2, vertexCount, 0.0001f);
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(0, 2).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(0, 2).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(0, 2).size(), vertexCount);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(0, 2), bsxs3, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(0, 2), bsys3, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(0, 2), bszs3, vertexCount, 0.0001f);
}

TEST(ConvertUnitsCommandTest, RadiansToDegreesMToCM) {
    ConvertibleDNAReader fixtures{dnac::TranslationUnit::m, dnac::RotationUnit::radians};
    auto output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);

    ASSERT_EQ(output->getTranslationUnit(), dnac::TranslationUnit::m);
    ASSERT_EQ(output->getRotationUnit(), dnac::RotationUnit::radians);

    dnac::ConvertUnitsCommand cmd(dnac::TranslationUnit::cm, dnac::RotationUnit::degrees);
    cmd.run(output.get());

    ASSERT_EQ(output->getTranslationUnit(), dnac::TranslationUnit::cm);
    ASSERT_EQ(output->getRotationUnit(), dnac::RotationUnit::degrees);

    float jtxs[] = {100.0f, 250.0f};
    float jtys[] = {300.0f, 450.0f};
    float jtzs[] = {400.0f, 800.0f};
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationXs(), jtxs, 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationYs(), jtys, 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationZs(), jtzs, 2ul, 0.0001f);

    float jrxs[] = {57.2958f, 143.2395f};
    float jrys[] = {171.8874f, 257.8311f};
    float jrzs[] = {229.1832f, 458.3664f};
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointRotationXs(), jrxs, 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointRotationYs(), jrys, 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointRotationZs(), jrzs, 2ul, 0.0001f);

    float jgvs[] = {50.0f,
                    20.0f,
                    30.0f,
                    25.0f,
                    40.0f,
                    15.0f,
                    5.72958f,
                    5.72958f,
                    51.56622f,
                    5.72958f,
                    42.97185f,
                    57.2958f,
                    0.3f,
                    0.7f,
                    0.45f};
    ASSERT_ELEMENTS_NEAR(output->getJointGroupValues(0), jgvs, 15ul, 0.0001f);

    const std::size_t vertexCount = 5ul;

    float vxs[] = {400.0f, 1200.0f, 2350.0f, -400.0f, 200.0f};
    float vys[] = {100.0f, 200.0f, 300.0f, 400.0f, 500.0f};
    float vzs[] = {1100.0f, -550.0f, 2200.0f, 300.0f, 610.0f};
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionXs(0), vxs, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionYs(0), vys, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionZs(0), vzs, vertexCount, 0.0001f);

    float bsxs1[] = {100.0f, 100.0f, 100.0f, 100.0f, 100.0f};
    float bsys1[] = {200.0f, 200.0f, 200.0f, 200.0f, 200.0f};
    float bszs1[] = {300.0f, 300.0f, 300.0f, 300.0f, 300.0f};

    float bsxs2[] = {400.0f, 400.0f, 400.0f, 400.0f, 400.0f};
    float bsys2[] = {500.0f, 500.0f, 500.0f, 500.0f, 500.0f};
    float bszs2[] = {600.0f, 600.0f, 600.0f, 600.0f, 600.0f};

    float bsxs3[] = {700.0f, 700.0f, 700.0f, 700.0f, 700.0f};
    float bsys3[] = {800.0f, 800.0f, 800.0f, 800.0f, 800.0f};
    float bszs3[] = {900.0f, 900.0f, 900.0f, 900.0f, 900.0f};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(0, 0).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(0, 0).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(0, 0).size(), vertexCount);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(0, 0), bsxs1, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(0, 0), bsys1, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(0, 0), bszs1, vertexCount, 0.0001f);
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(0, 1).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(0, 1).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(0, 1).size(), vertexCount);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(0, 1), bsxs2, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(0, 1), bsys2, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(0, 1), bszs2, vertexCount, 0.0001f);
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(0, 2).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(0, 2).size(), vertexCount);
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(0, 2).size(), vertexCount);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(0, 2), bsxs3, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(0, 2), bsys3, vertexCount, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(0, 2), bszs3, vertexCount, 0.0001f);
}
