// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"

#include <cstdint>

namespace {

class PrunableDNAReader : public dna::FakeDNACReader {
public:
    explicit PrunableDNAReader(dnac::MemoryResource* memRes = nullptr) :
        blendShapeTargetDeltas{memRes} {

        float bxs[] = {0.0005f, 0.0015f, 0.002f, 0.005f, 0.01f, 0.001f, 0.1f};
        float bys[] = {0.0005f, 0.0015f, 0.002f, 0.005f, 0.01f, 0.001f, 0.1f};
        float bzs[] = {0.0005f, 0.0015f, 0.002f, 0.005f, 0.01f, 0.001f, 0.1f};
        blendShapeTargetDeltas.xs.assign(bxs, bxs + 7ul);
        blendShapeTargetDeltas.ys.assign(bys, bys + 7ul);
        blendShapeTargetDeltas.zs.assign(bzs, bzs + 7ul);

        std::uint32_t indices[] = {0u, 1u, 2u, 3u, 4u, 5u, 6u};
        blendShapeTargetVertexIndices.assign(indices, indices + 7ul);
    }

    std::uint16_t getMeshCount() const override {
        return 1u;
    }

    dnac::StringView getMeshName(std::uint16_t /*unused*/) const override {
        return dnac::StringView{"M", 1ul};
    }

    std::uint16_t getBlendShapeTargetCount(std::uint16_t /*unused*/) const override {
        return 1u;
    }

    std::uint32_t getBlendShapeTargetDeltaCount(std::uint16_t /*unused*/, std::uint16_t /*unused*/) const override {
        return static_cast<std::uint32_t>(blendShapeTargetDeltas.size());
    }

    dnac::Vector3 getBlendShapeTargetDelta(std::uint16_t /*unused*/,
                                           std::uint16_t /*unused*/,
                                           std::uint32_t deltaIndex) const override {
        return dnac::Vector3{blendShapeTargetDeltas.xs[deltaIndex],
                             blendShapeTargetDeltas.ys[deltaIndex],
                             blendShapeTargetDeltas.zs[deltaIndex]};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t /*unused*/, std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{blendShapeTargetDeltas.xs};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t /*unused*/, std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{blendShapeTargetDeltas.ys};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t /*unused*/, std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{blendShapeTargetDeltas.zs};
    }

    dnac::ConstArrayView<std::uint32_t> getBlendShapeTargetVertexIndices(std::uint16_t /*unused*/,
                                                                         std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<std::uint32_t>{blendShapeTargetVertexIndices};
    }

private:
    dnac::DynArray<std::uint32_t> blendShapeTargetVertexIndices;
    dnac::RawVector3Vector blendShapeTargetDeltas;
};

class PruneBlendShapeTargetsCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        PrunableDNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
        threshold = 0.002f;

        float bxs[] = {0.0015f, 0.002f, 0.005f, 0.01f, 0.1f};
        float bys[] = {0.0015f, 0.002f, 0.005f, 0.01f, 0.1f};
        float bzs[] = {0.0015f, 0.002f, 0.005f, 0.01f, 0.1f};
        expectedBlendShapeTargetDeltaXs.assign(bxs, bxs + 5ul);
        expectedBlendShapeTargetDeltaYs.assign(bys, bys + 5ul);
        expectedBlendShapeTargetDeltaZs.assign(bzs, bzs + 5ul);

        std::uint32_t indices[] = {1u, 2u, 3u, 4u, 6u};
        expectedBlendShapeTargetVertexIndices.assign(indices, indices + 5ul);
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;

    float threshold;

    dnac::Vector<float> expectedBlendShapeTargetDeltaXs;
    dnac::Vector<float> expectedBlendShapeTargetDeltaYs;
    dnac::Vector<float> expectedBlendShapeTargetDeltaZs;
    dnac::Vector<std::uint32_t> expectedBlendShapeTargetVertexIndices;
};

}  // namespace

TEST_F(PruneBlendShapeTargetsCommandTest, CutElementsBelowThreshold) {
    dnac::PruneBlendShapeTargetsCommand pruneCmd(threshold);
    pruneCmd.run(output.get());

    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(0, 0),
                         expectedBlendShapeTargetDeltaXs,
                         expectedBlendShapeTargetDeltaXs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(0, 0),
                         expectedBlendShapeTargetDeltaYs,
                         expectedBlendShapeTargetDeltaYs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(0, 0),
                         expectedBlendShapeTargetDeltaZs,
                         expectedBlendShapeTargetDeltaZs.size(),
                         0.0001);

    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(0, 0),
                       expectedBlendShapeTargetVertexIndices,
                       expectedBlendShapeTargetVertexIndices.size());
}
