// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/AnimatedMapDNAReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"

#include <cstdint>
#include <numeric>
#include <vector>

namespace {

class RemoveAnimatedMapCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        AnimatedMapDNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;
};

}  // namespace

TEST_F(RemoveAnimatedMapCommandTest, RemoveSingleAnimatedMap) {
    dnac::RemoveAnimatedMapCommand cmd(2u);

    ASSERT_EQ(output->getLODCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getAnimatedMapCount(), 5u);
    ASSERT_STREQ(output->getAnimatedMapName(0).c_str(), "animatedMap1");
    ASSERT_STREQ(output->getAnimatedMapName(1).c_str(), "animatedMap2");
    ASSERT_STREQ(output->getAnimatedMapName(2).c_str(), "animatedMap3");
    ASSERT_STREQ(output->getAnimatedMapName(3).c_str(), "animatedMap4");
    ASSERT_STREQ(output->getAnimatedMapName(4).c_str(), "animatedMap5");
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(0).size(), 5u);
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(1).size(), 3u);

    // Check behavior
    ASSERT_EQ(output->getAnimatedMapLODs()[0], 12u);
    ASSERT_EQ(output->getAnimatedMapLODs()[1], 8u);
    const std::array<std::uint16_t, 12u> initialInputIndices = {1u, 263u, 21u, 320u, 2u, 20u, 319u, 3u, 21u, 320u, 4u, 5u};
    ASSERT_EQ(output->getAnimatedMapInputIndices().size(), initialInputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getAnimatedMapInputIndices(), initialInputIndices, initialInputIndices.size());
    const std::array<std::uint16_t, 12u> initialOutputIndices = {0u, 0u, 0u, 0u, 1u, 1u, 1u, 2u, 2u, 2u, 3u, 4u};
    ASSERT_EQ(output->getAnimatedMapOutputIndices().size(), initialOutputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getAnimatedMapOutputIndices(), initialOutputIndices, initialOutputIndices.size());
    const std::array<float, 12u> initialCutValues =
        {0.0f, 0.0f, 0.0f, -0.066667f, 0.0f, 0.0f, -0.1f, 0.0f, 0.0f, -0.1f, -0.333333f, -0.333333f};
    ASSERT_EQ(output->getAnimatedMapCutValues().size(), initialCutValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapCutValues(), initialCutValues, initialCutValues.size(), 1e-5f);
    const std::array<float, 12u> initialSlopeValues =
        {1.0f, -1.0f, 1.0f, 0.266667f, 1.0f, 0.5f, 0.4f, 1.0f, 0.5f, 0.4f, 1.333333f, 1.333333f};
    ASSERT_EQ(output->getAnimatedMapSlopeValues().size(), initialSlopeValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapSlopeValues(), initialSlopeValues, initialSlopeValues.size(), 1e-5f);
    const std::array<float, 12u> initialFromValues =
        {0.0f, 0.0f, 0.0f, 0.25f, 0.0f, 0.0f, 0.25f, 0.0f, 0.0f, 0.25f, 0.25f, 0.25f};
    ASSERT_EQ(output->getAnimatedMapFromValues().size(), initialFromValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapFromValues(), initialFromValues, initialFromValues.size(), 1e-5f);
    const std::array<float, 12u> initialToValues = {1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f};
    ASSERT_EQ(output->getAnimatedMapToValues().size(), initialToValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapToValues(), initialToValues, initialToValues.size(), 1e-5f);

    // Remove animated map "animatedMap3"
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getAnimatedMapCount(), 4u);
    ASSERT_STREQ(output->getAnimatedMapName(0).c_str(), "animatedMap1");
    ASSERT_STREQ(output->getAnimatedMapName(1).c_str(), "animatedMap2");
    ASSERT_STREQ(output->getAnimatedMapName(2).c_str(), "animatedMap4");
    ASSERT_STREQ(output->getAnimatedMapName(3).c_str(), "animatedMap5");
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(0).size(), 4u);
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(1).size(), 2u);

    // Check behavior
    ASSERT_EQ(output->getAnimatedMapLODs()[0], 9u);
    ASSERT_EQ(output->getAnimatedMapLODs()[1], 5u);
    const std::array<std::uint16_t, 9u> expectedInputIndices = {1u, 263u, 21u, 320u, 2u, 20u, 319u, 4u, 5u};
    ASSERT_EQ(output->getAnimatedMapInputIndices().size(), expectedInputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getAnimatedMapInputIndices(), expectedInputIndices, expectedInputIndices.size());
    const std::array<std::uint16_t, 9u> expectedOutputIndices = {0u, 0u, 0u, 0u, 1u, 1u, 1u, 2u, 3u};
    ASSERT_EQ(output->getAnimatedMapOutputIndices().size(), expectedOutputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getAnimatedMapOutputIndices(), expectedOutputIndices, expectedOutputIndices.size());
    const std::array<float, 9u> expectedCutValues = {0.0f, 0.0f, 0.0f, -0.066667f, 0.0f, 0.0f, -0.1f, -0.333333f, -0.333333f};
    ASSERT_EQ(output->getAnimatedMapCutValues().size(), expectedCutValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapCutValues(), expectedCutValues, expectedCutValues.size(), 1e-5f);
    const std::array<float, 9u> expectedSlopeValues = {1.0f, -1.0f, 1.0f, 0.266667f, 1.0f, 0.5f, 0.4f, 1.333333f, 1.333333f};
    ASSERT_EQ(output->getAnimatedMapSlopeValues().size(), expectedSlopeValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapSlopeValues(), expectedSlopeValues, expectedSlopeValues.size(), 1e-5f);
    const std::array<float, 9u> expectedFromValues = {0.0f, 0.0f, 0.0f, 0.25f, 0.0f, 0.0f, 0.25f, 0.25f, 0.25f};
    ASSERT_EQ(output->getAnimatedMapFromValues().size(), expectedFromValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapFromValues(), expectedFromValues, expectedFromValues.size(), 1e-5f);
    const std::array<float, 9u> expectedToValues = {1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f};
    ASSERT_EQ(output->getAnimatedMapToValues().size(), expectedToValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapToValues(), expectedToValues, expectedToValues.size(), 1e-5f);
}

TEST_F(RemoveAnimatedMapCommandTest, RemoveMultipleAnimatedMaps) {
    std::vector<std::uint16_t> animatedMapIndices{1u, 3u};
    dnac::RemoveAnimatedMapCommand cmd;
    cmd.setAnimatedMapIndices(dnac::ConstArrayView<std::uint16_t>{animatedMapIndices});

    ASSERT_EQ(output->getLODCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getAnimatedMapCount(), 5u);
    ASSERT_STREQ(output->getAnimatedMapName(0).c_str(), "animatedMap1");
    ASSERT_STREQ(output->getAnimatedMapName(1).c_str(), "animatedMap2");
    ASSERT_STREQ(output->getAnimatedMapName(2).c_str(), "animatedMap3");
    ASSERT_STREQ(output->getAnimatedMapName(3).c_str(), "animatedMap4");
    ASSERT_STREQ(output->getAnimatedMapName(4).c_str(), "animatedMap5");
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(0).size(), 5u);
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(1).size(), 3u);

    // Check behavior
    ASSERT_EQ(output->getAnimatedMapLODs()[0], 12u);
    ASSERT_EQ(output->getAnimatedMapLODs()[1], 8u);
    const std::array<std::uint16_t, 12u> initialInputIndices = {1u, 263u, 21u, 320u, 2u, 20u, 319u, 3u, 21u, 320u, 4u, 5u};
    ASSERT_EQ(output->getAnimatedMapInputIndices().size(), initialInputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getAnimatedMapInputIndices(), initialInputIndices, initialInputIndices.size());
    const std::array<std::uint16_t, 12u> initialOutputIndices = {0u, 0u, 0u, 0u, 1u, 1u, 1u, 2u, 2u, 2u, 3u, 4u};
    ASSERT_EQ(output->getAnimatedMapOutputIndices().size(), initialOutputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getAnimatedMapOutputIndices(), initialOutputIndices, initialOutputIndices.size());
    const std::array<float, 12u> initialCutValues =
        {0.0f, 0.0f, 0.0f, -0.066667f, 0.0f, 0.0f, -0.1f, 0.0f, 0.0f, -0.1f, -0.333333f, -0.333333f};
    ASSERT_EQ(output->getAnimatedMapCutValues().size(), initialCutValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapCutValues(), initialCutValues, initialCutValues.size(), 1e-5f);
    const std::array<float, 12u> initialSlopeValues =
        {1.0f, -1.0f, 1.0f, 0.266667f, 1.0f, 0.5f, 0.4f, 1.0f, 0.5f, 0.4f, 1.333333f, 1.333333f};
    ASSERT_EQ(output->getAnimatedMapSlopeValues().size(), initialSlopeValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapSlopeValues(), initialSlopeValues, initialSlopeValues.size(), 1e-5f);
    const std::array<float, 12u> initialFromValues =
        {0.0f, 0.0f, 0.0f, 0.25f, 0.0f, 0.0f, 0.25f, 0.0f, 0.0f, 0.25f, 0.25f, 0.25f};
    ASSERT_EQ(output->getAnimatedMapFromValues().size(), initialFromValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapFromValues(), initialFromValues, initialFromValues.size(), 1e-5f);
    const std::array<float, 12u> initialToValues = {1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f};
    ASSERT_EQ(output->getAnimatedMapToValues().size(), initialToValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapToValues(), initialToValues, initialToValues.size(), 1e-5f);

    // Remove animated maps "animatedMap2" and "animatedMap4"
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getAnimatedMapCount(), 3u);
    ASSERT_STREQ(output->getAnimatedMapName(0).c_str(), "animatedMap1");
    ASSERT_STREQ(output->getAnimatedMapName(1).c_str(), "animatedMap3");
    ASSERT_STREQ(output->getAnimatedMapName(2).c_str(), "animatedMap5");
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(0).size(), 3u);
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(1).size(), 2u);

    // Check behavior
    ASSERT_EQ(output->getAnimatedMapLODs()[0], 8u);
    ASSERT_EQ(output->getAnimatedMapLODs()[1], 7u);

    const std::array<std::uint16_t, 8u> expectedInputIndices = {1u, 263u, 21u, 320u, 3u, 21u, 320u, 5u};
    ASSERT_EQ(output->getAnimatedMapInputIndices().size(), expectedInputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getAnimatedMapInputIndices(), expectedInputIndices, expectedInputIndices.size());
    const std::array<std::uint16_t, 8u> expectedOutputIndices = {0u, 0u, 0u, 0u, 1u, 1u, 1u, 2u};
    ASSERT_EQ(output->getAnimatedMapOutputIndices().size(), expectedOutputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getAnimatedMapOutputIndices(), expectedOutputIndices, expectedOutputIndices.size());
    const std::array<float, 8u> expectedCutValues = {0.0f, 0.0f, 0.0f, -0.066667f, 0.0f, 0.0f, -0.1f, -0.333333f};
    ASSERT_EQ(output->getAnimatedMapCutValues().size(), expectedCutValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapCutValues(), expectedCutValues, expectedCutValues.size(), 1e-5f);
    const std::array<float, 8u> expectedSlopeValues = {1.0f, -1.0f, 1.0f, 0.266667f, 1.0f, 0.5f, 0.4f, 1.333333f};
    ASSERT_EQ(output->getAnimatedMapSlopeValues().size(), expectedSlopeValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapSlopeValues(), expectedSlopeValues, expectedSlopeValues.size(), 1e-5f);
    const std::array<float, 8u> expectedFromValues = {0.0f, 0.0f, 0.0f, 0.25f, 0.0f, 0.0f, 0.25f, 0.25f};
    ASSERT_EQ(output->getAnimatedMapFromValues().size(), expectedFromValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapFromValues(), expectedFromValues, expectedFromValues.size(), 1e-5f);
    const std::array<float, 8u> expectedToValues = {1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f};
    ASSERT_EQ(output->getAnimatedMapToValues().size(), expectedToValues.size());
    ASSERT_ELEMENTS_NEAR(output->getAnimatedMapToValues(), expectedToValues, expectedToValues.size(), 1e-5f);
}

TEST_F(RemoveAnimatedMapCommandTest, RemoveAllAnimatedMapsOneByOne) {
    const auto animatedMapCount = output->getAnimatedMapCount();
    dnac::RemoveAnimatedMapCommand cmd;
    for (std::uint16_t animatedMapIndex = 0u; animatedMapIndex < animatedMapCount; ++animatedMapIndex) {
        cmd.setAnimatedMapIndex(0);  // Due to remapping, 0, 1, 2 wouldn't remove all, as after removing 0th, 2nd would become 1st
        cmd.run(output.get());
    }

    ASSERT_EQ(output->getLODCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getAnimatedMapCount(), 0u);
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(0).size(), 0u);
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(1).size(), 0u);

    // Check behavior
    ASSERT_EQ(output->getAnimatedMapLODs()[0], 0u);
    ASSERT_EQ(output->getAnimatedMapLODs()[1], 0u);
    ASSERT_EQ(output->getAnimatedMapInputIndices().size(), 0u);
    ASSERT_EQ(output->getAnimatedMapOutputIndices().size(), 0u);
    ASSERT_EQ(output->getAnimatedMapCutValues().size(), 0u);
    ASSERT_EQ(output->getAnimatedMapSlopeValues().size(), 0u);
    ASSERT_EQ(output->getAnimatedMapFromValues().size(), 0u);
    ASSERT_EQ(output->getAnimatedMapToValues().size(), 0u);
}

TEST_F(RemoveAnimatedMapCommandTest, RemoveAllAnimatedMaps) {
    const auto animatedMapCount = output->getAnimatedMapCount();
    std::vector<std::uint16_t> animatedMapsToRemove;
    animatedMapsToRemove.resize(animatedMapCount);
    std::iota(animatedMapsToRemove.begin(), animatedMapsToRemove.end(), static_cast<std::uint16_t>(0u));
    dnac::RemoveAnimatedMapCommand cmd{dnac::ConstArrayView<std::uint16_t>{animatedMapsToRemove}};
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getAnimatedMapCount(), 0u);
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(0).size(), 0u);
    ASSERT_EQ(output->getAnimatedMapIndicesForLOD(1).size(), 0u);

    // Check behavior
    ASSERT_EQ(output->getAnimatedMapLODs()[0], 0u);
    ASSERT_EQ(output->getAnimatedMapLODs()[1], 0u);
    ASSERT_EQ(output->getAnimatedMapInputIndices().size(), 0u);
    ASSERT_EQ(output->getAnimatedMapOutputIndices().size(), 0u);
    ASSERT_EQ(output->getAnimatedMapCutValues().size(), 0u);
    ASSERT_EQ(output->getAnimatedMapSlopeValues().size(), 0u);
    ASSERT_EQ(output->getAnimatedMapFromValues().size(), 0u);
    ASSERT_EQ(output->getAnimatedMapToValues().size(), 0u);
}
