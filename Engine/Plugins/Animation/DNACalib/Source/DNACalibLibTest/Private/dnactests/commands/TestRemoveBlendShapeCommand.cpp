// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/BlendShapeDNAReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"

#include <cstdint>
#include <numeric>
#include <vector>

namespace {

class RemoveBlendShapeCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        BlendShapeDNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;
};

}  // namespace

TEST_F(RemoveBlendShapeCommandTest, RemoveSingleBlendShape) {
    dnac::RemoveBlendShapeCommand cmd(1u);

    ASSERT_EQ(output->getLODCount(), 2u);
    ASSERT_EQ(output->getMeshCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getBlendShapeChannelCount(), 4u);
    ASSERT_STREQ(output->getBlendShapeChannelName(0).c_str(), "blendshape1");
    ASSERT_STREQ(output->getBlendShapeChannelName(1).c_str(), "blendshape2");
    ASSERT_STREQ(output->getBlendShapeChannelName(2).c_str(), "blendshape3");
    ASSERT_STREQ(output->getBlendShapeChannelName(3).c_str(), "blendshape4");
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(0).size(), 4u);
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(1).size(), 2u);

    // Check behavior
    ASSERT_EQ(output->getBlendShapeChannelLODs()[0], 4u);
    ASSERT_EQ(output->getBlendShapeChannelLODs()[1], 2u);
    const std::array<std::uint16_t, 4u> initialInputIndices = {0u, 0u, 1u, 1u};
    ASSERT_EQ(output->getBlendShapeChannelInputIndices().size(), initialInputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeChannelInputIndices(), initialInputIndices, initialInputIndices.size());
    const std::array<std::uint16_t, 4u> initialOutputIndices = {1u, 0u, 2u, 3u};
    ASSERT_EQ(output->getBlendShapeChannelOutputIndices().size(), initialOutputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeChannelOutputIndices(), initialOutputIndices, initialOutputIndices.size());

    // Check geometry
    ASSERT_EQ(output->getBlendShapeTargetCount(0), 3u);
    ASSERT_EQ(output->getBlendShapeTargetCount(1), 1u);

    // Remove blend shape "blendshape2"
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 2u);
    ASSERT_EQ(output->getMeshCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getBlendShapeChannelCount(), 3u);
    ASSERT_STREQ(output->getBlendShapeChannelName(0).c_str(), "blendshape1");
    ASSERT_STREQ(output->getBlendShapeChannelName(1).c_str(), "blendshape3");
    ASSERT_STREQ(output->getBlendShapeChannelName(2).c_str(), "blendshape4");
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(0).size(), 3u);
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(1).size(), 2u);

    // Check behavior
    ASSERT_EQ(output->getBlendShapeChannelLODs()[0], 3u);
    ASSERT_EQ(output->getBlendShapeChannelLODs()[1], 2u);
    const std::array<std::uint16_t, 3u> expectedInputIndices = {0u, 1u, 1u};
    ASSERT_EQ(output->getBlendShapeChannelInputIndices().size(), expectedInputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeChannelInputIndices(), expectedInputIndices, expectedInputIndices.size());
    const std::array<std::uint16_t, 3u> expectedOutputIndices = {0u, 1u, 2u};
    ASSERT_EQ(output->getBlendShapeChannelOutputIndices().size(), expectedOutputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeChannelOutputIndices(), expectedOutputIndices, expectedOutputIndices.size());

    // Check geometry
    ASSERT_EQ(output->getBlendShapeTargetCount(0), 2u);
    ASSERT_EQ(output->getBlendShapeTargetCount(1), 1u);
}

TEST_F(RemoveBlendShapeCommandTest, RemoveMultipleBlendShapes) {
    std::vector<std::uint16_t> blendShapeIndices{1u, 2u};
    dnac::RemoveBlendShapeCommand cmd(dnac::ConstArrayView<std::uint16_t>{blendShapeIndices});

    ASSERT_EQ(output->getLODCount(), 2u);
    ASSERT_EQ(output->getMeshCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getBlendShapeChannelCount(), 4u);
    ASSERT_STREQ(output->getBlendShapeChannelName(0).c_str(), "blendshape1");
    ASSERT_STREQ(output->getBlendShapeChannelName(1).c_str(), "blendshape2");
    ASSERT_STREQ(output->getBlendShapeChannelName(2).c_str(), "blendshape3");
    ASSERT_STREQ(output->getBlendShapeChannelName(3).c_str(), "blendshape4");
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(0).size(), 4u);
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(1).size(), 2u);

    // Check behavior
    ASSERT_EQ(output->getBlendShapeChannelLODs()[0], 4u);
    ASSERT_EQ(output->getBlendShapeChannelLODs()[1], 2u);
    const std::array<std::uint16_t, 4u> initialInputIndices = {0u, 0u, 1u, 1u};
    ASSERT_EQ(output->getBlendShapeChannelInputIndices().size(), initialInputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeChannelInputIndices(), initialInputIndices, initialInputIndices.size());
    const std::array<std::uint16_t, 4u> initialOutputIndices = {1u, 0u, 2u, 3u};
    ASSERT_EQ(output->getBlendShapeChannelOutputIndices().size(), initialOutputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeChannelOutputIndices(), initialOutputIndices, initialOutputIndices.size());

    // Check geometry
    ASSERT_EQ(output->getBlendShapeTargetCount(0), 3u);
    ASSERT_EQ(output->getBlendShapeTargetCount(1), 1u);

    // Remove blend shapes "blendshape2" and "blendshape3"
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 2u);
    ASSERT_EQ(output->getMeshCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getBlendShapeChannelCount(), 2u);
    ASSERT_STREQ(output->getBlendShapeChannelName(0).c_str(), "blendshape1");
    ASSERT_STREQ(output->getBlendShapeChannelName(1).c_str(), "blendshape4");
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(0).size(), 2u);
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(1).size(), 1u);

    // Check behavior
    ASSERT_EQ(output->getBlendShapeChannelLODs()[0], 2u);
    ASSERT_EQ(output->getBlendShapeChannelLODs()[1], 1u);
    const std::array<std::uint16_t, 2u> expectedInputIndices = {0u, 1u};
    ASSERT_EQ(output->getBlendShapeChannelInputIndices().size(), expectedInputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeChannelInputIndices(), expectedInputIndices, expectedInputIndices.size());
    const std::array<std::uint16_t, 2u> expectedOutputIndices = {0u, 1u};
    ASSERT_EQ(output->getBlendShapeChannelOutputIndices().size(), expectedOutputIndices.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeChannelOutputIndices(), expectedOutputIndices, expectedOutputIndices.size());

    // Check geometry
    ASSERT_EQ(output->getBlendShapeTargetCount(0), 1u);
    ASSERT_EQ(output->getBlendShapeTargetCount(1), 1u);
}

TEST_F(RemoveBlendShapeCommandTest, RemoveAllBlendShapesOneByOne) {
    const auto blendShapeCount = output->getBlendShapeChannelCount();
    dnac::RemoveBlendShapeCommand cmd;
    for (std::uint16_t blendShapeIndex = 0u; blendShapeIndex < blendShapeCount; ++blendShapeIndex) {
        cmd.setBlendShapeIndex(0);  // Due to remapping, 0, 1, 2 wouldn't remove all, as after removing 0th, 2nd would become 1st
        cmd.run(output.get());
    }

    ASSERT_EQ(output->getLODCount(), 2u);
    ASSERT_EQ(output->getMeshCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getBlendShapeChannelCount(), 0u);
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(0).size(), 0u);
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(1).size(), 0u);

    // Check behavior
    ASSERT_EQ(output->getBlendShapeChannelLODs()[0], 0u);
    ASSERT_EQ(output->getBlendShapeChannelLODs()[1], 0u);
    ASSERT_EQ(output->getBlendShapeChannelInputIndices().size(), 0u);
    ASSERT_EQ(output->getBlendShapeChannelOutputIndices().size(), 0u);

    // Check geometry
    ASSERT_EQ(output->getBlendShapeTargetCount(0), 0u);
    ASSERT_EQ(output->getBlendShapeTargetCount(1), 0u);
}

TEST_F(RemoveBlendShapeCommandTest, RemoveAllBlendShapes) {
    const auto blendShapeCount = output->getBlendShapeChannelCount();
    std::vector<std::uint16_t> blendShapesToRemove;
    blendShapesToRemove.resize(blendShapeCount);
    std::iota(blendShapesToRemove.begin(), blendShapesToRemove.end(), static_cast<std::uint16_t>(0u));
    dnac::RemoveBlendShapeCommand cmd;
    cmd.setBlendShapeIndices(dnac::ConstArrayView<std::uint16_t>{blendShapesToRemove});
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 2u);
    ASSERT_EQ(output->getMeshCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getBlendShapeChannelCount(), 0u);
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(0).size(), 0u);
    ASSERT_EQ(output->getBlendShapeChannelIndicesForLOD(1).size(), 0u);

    // Check behavior
    ASSERT_EQ(output->getBlendShapeChannelLODs()[0], 0u);
    ASSERT_EQ(output->getBlendShapeChannelLODs()[1], 0u);
    ASSERT_EQ(output->getBlendShapeChannelInputIndices().size(), 0u);
    ASSERT_EQ(output->getBlendShapeChannelOutputIndices().size(), 0u);

    // Check geometry
    ASSERT_EQ(output->getBlendShapeTargetCount(0), 0u);
    ASSERT_EQ(output->getBlendShapeTargetCount(1), 0u);
}
