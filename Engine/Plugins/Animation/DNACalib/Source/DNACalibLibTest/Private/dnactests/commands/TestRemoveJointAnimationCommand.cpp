// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/JointDNAReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"

#include <cstdint>

namespace {

class RemoveJointAnimationCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        JointDNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
        jointIndex = 1u;
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;
    std::uint16_t jointIndex;
};

}  // namespace

TEST_F(RemoveJointAnimationCommandTest, RemoveSingleJointAnimation) {
    dnac::RemoveJointAnimationCommand cmd(jointIndex);
    cmd.run(output.get());

    ASSERT_EQ(output->getJointCount(), 4u);

    static const char* expectedJointNames[] = {"JA", "JB", "JC", "JD"};
    ASSERT_EQ(output->getJointName(0), (dnac::StringView{expectedJointNames[0], 2ul}));
    ASSERT_EQ(output->getJointName(1), (dnac::StringView{expectedJointNames[1], 2ul}));
    ASSERT_EQ(output->getJointName(2), (dnac::StringView{expectedJointNames[2], 2ul}));
    ASSERT_EQ(output->getJointName(3), (dnac::StringView{expectedJointNames[3], 2ul}));

    static const std::uint16_t expectedJointIndicesPerLOD0[] = {0u, 1u, 2u};
    static const std::uint16_t expectedJointIndicesPerLOD1[] = {0u, 1u};
    ASSERT_ELEMENTS_EQ(output->getJointIndicesForLOD(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndicesPerLOD0, 3ul}),
                       3ul);
    ASSERT_ELEMENTS_EQ(output->getJointIndicesForLOD(1),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndicesPerLOD1, 2ul}),
                       2ul);

    ASSERT_EQ(output->getJointParentIndex(0), 0);
    ASSERT_EQ(output->getJointParentIndex(1), 0);
    ASSERT_EQ(output->getJointParentIndex(2), 1);
    ASSERT_EQ(output->getJointParentIndex(3), 2);
    ASSERT_EQ(output->getJointParentIndex(4), static_cast<std::uint16_t>(-1));

    static const float expectedNeutralXs[] = {1.0f, 4.0f, 7.0f, 10.0f};
    static const float expectedNeutralYs[] = {2.0f, 5.0f, 8.0f, 11.0f};
    static const float expectedNeutralZs[] = {3.0f, 6.0f, 9.0f, 12.0f};
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationXs(), (dnac::ConstArrayView<float>{expectedNeutralXs, 4ul}), 4ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationYs(), (dnac::ConstArrayView<float>{expectedNeutralYs, 4ul}), 4ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationZs(), (dnac::ConstArrayView<float>{expectedNeutralZs, 4ul}), 4ul);

    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationXs(), (dnac::ConstArrayView<float>{expectedNeutralXs, 4ul}), 4ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationYs(), (dnac::ConstArrayView<float>{expectedNeutralYs, 4ul}), 4ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationZs(), (dnac::ConstArrayView<float>{expectedNeutralZs, 4ul}), 4ul);

    ASSERT_EQ(output->getJointGroupCount(), 1u);

    static const std::uint16_t expectedJointIndices[] = {0u, 1u, 2u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupJointIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndices, 3ul}),
                       3ul);

    static const std::uint16_t expectedLODs[] = {2u, 1u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupLODs(0), (dnac::ConstArrayView<std::uint16_t>{expectedLODs, 2ul}), 2ul);

    static const std::uint16_t expectedInputIndices[] = {13, 56, 120};
    ASSERT_ELEMENTS_EQ(output->getJointGroupInputIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedInputIndices, 3ul}),
                       3ul);

    static const std::uint16_t expectedOutputIndices[] = {8, 18};
    ASSERT_ELEMENTS_EQ(output->getJointGroupOutputIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedOutputIndices, 2ul}),
                       2ul);

    static const float expectedValues[] = {0.5f, 0.2f, 0.3f, 0.1f, 0.75f, 1.0f};
    ASSERT_ELEMENTS_EQ(output->getJointGroupValues(0), (dnac::ConstArrayView<float>{expectedValues, 6ul}), 6ul);

    ASSERT_EQ(output->getSkinWeightsCount(0), 4u);

    static const std::uint16_t expectedSWJointIndices0[] = {0, 1, 2};
    static const std::uint16_t expectedSWJointIndices1[] = {0, 1};
    static const std::uint16_t expectedSWJointIndices2[] = {1, 2};
    static const std::uint16_t expectedSWJointIndices3[] = {1};

    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices0, 3ul}),
                       3ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 1),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices1, 2ul}),
                       2ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 2),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices2, 2ul}),
                       2ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 3),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices3, 1ul}),
                       1ul);

    static const float expectedSWValues0[] = {0.1f, 0.7f, 0.2f};
    static const float expectedSWValues1[] = {0.2f, 0.8f};
    static const float expectedSWValues2[] = {0.4f, 0.6f};
    static const float expectedSWValues3[] = {1.0f};

    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 0), (dnac::ConstArrayView<float>{expectedSWValues0, 3ul}), 3ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 1), (dnac::ConstArrayView<float>{expectedSWValues1, 2ul}), 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 2), (dnac::ConstArrayView<float>{expectedSWValues2, 2ul}), 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 3), (dnac::ConstArrayView<float>{expectedSWValues3, 1ul}), 1ul, 0.0001f);
}

TEST_F(RemoveJointAnimationCommandTest, RemoveMultipleJointAnimations) {
    std::vector<std::uint16_t> jointIndices{1u, 2u};
    dnac::RemoveJointAnimationCommand cmd(dnac::ConstArrayView<std::uint16_t>{jointIndices});
    cmd.run(output.get());

    ASSERT_EQ(output->getJointCount(), 4u);

    static const char* expectedJointNames[] = {"JA", "JB", "JC", "JD"};
    ASSERT_EQ(output->getJointName(0), (dnac::StringView{expectedJointNames[0], 2ul}));
    ASSERT_EQ(output->getJointName(1), (dnac::StringView{expectedJointNames[1], 2ul}));
    ASSERT_EQ(output->getJointName(2), (dnac::StringView{expectedJointNames[2], 2ul}));
    ASSERT_EQ(output->getJointName(3), (dnac::StringView{expectedJointNames[3], 2ul}));

    static const std::uint16_t expectedJointIndicesPerLOD0[] = {0u, 1u, 2u};
    static const std::uint16_t expectedJointIndicesPerLOD1[] = {0u, 1u};
    ASSERT_ELEMENTS_EQ(output->getJointIndicesForLOD(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndicesPerLOD0, 3ul}),
                       3ul);
    ASSERT_ELEMENTS_EQ(output->getJointIndicesForLOD(1),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndicesPerLOD1, 2ul}),
                       2ul);

    ASSERT_EQ(output->getJointParentIndex(0), 0);
    ASSERT_EQ(output->getJointParentIndex(1), 0);
    ASSERT_EQ(output->getJointParentIndex(2), 1);
    ASSERT_EQ(output->getJointParentIndex(3), 2);
    ASSERT_EQ(output->getJointParentIndex(4), static_cast<std::uint16_t>(-1));

    static const float expectedNeutralXs[] = {1.0f, 4.0f, 7.0f, 10.0f};
    static const float expectedNeutralYs[] = {2.0f, 5.0f, 8.0f, 11.0f};
    static const float expectedNeutralZs[] = {3.0f, 6.0f, 9.0f, 12.0f};
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationXs(), (dnac::ConstArrayView<float>{expectedNeutralXs, 4ul}), 4ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationYs(), (dnac::ConstArrayView<float>{expectedNeutralYs, 4ul}), 4ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationZs(), (dnac::ConstArrayView<float>{expectedNeutralZs, 4ul}), 4ul);

    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationXs(), (dnac::ConstArrayView<float>{expectedNeutralXs, 4ul}), 4ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationYs(), (dnac::ConstArrayView<float>{expectedNeutralYs, 4ul}), 4ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationZs(), (dnac::ConstArrayView<float>{expectedNeutralZs, 4ul}), 4ul);

    ASSERT_EQ(output->getJointGroupCount(), 1u);

    static const std::uint16_t expectedJointIndices[] = {0u, 1u, 2u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupJointIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndices, 3ul}),
                       3ul);

    static const std::uint16_t expectedLODs[] = {1u, 1u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupLODs(0), (dnac::ConstArrayView<std::uint16_t>{expectedLODs, 2ul}), 2ul);

    static const std::uint16_t expectedInputIndices[] = {13, 56, 120};
    ASSERT_ELEMENTS_EQ(output->getJointGroupInputIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedInputIndices, 3ul}),
                       3ul);

    static const std::uint16_t expectedOutputIndices[] = {8};
    ASSERT_ELEMENTS_EQ(output->getJointGroupOutputIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedOutputIndices, 1ul}),
                       1ul);

    static const float expectedValues[] = {0.5f, 0.2f, 0.3f};
    ASSERT_ELEMENTS_EQ(output->getJointGroupValues(0), (dnac::ConstArrayView<float>{expectedValues, 3ul}), 3ul);

    ASSERT_EQ(output->getSkinWeightsCount(0), 4u);

    static const std::uint16_t expectedSWJointIndices0[] = {0, 1, 2};
    static const std::uint16_t expectedSWJointIndices1[] = {0, 1};
    static const std::uint16_t expectedSWJointIndices2[] = {1, 2};
    static const std::uint16_t expectedSWJointIndices3[] = {1};

    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices0, 3ul}),
                       3ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 1),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices1, 2ul}),
                       2ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 2),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices2, 2ul}),
                       2ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 3),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices3, 1ul}),
                       1ul);

    static const float expectedSWValues0[] = {0.1f, 0.7f, 0.2f};
    static const float expectedSWValues1[] = {0.2f, 0.8f};
    static const float expectedSWValues2[] = {0.4f, 0.6f};
    static const float expectedSWValues3[] = {1.0f};

    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 0), (dnac::ConstArrayView<float>{expectedSWValues0, 3ul}), 3ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 1), (dnac::ConstArrayView<float>{expectedSWValues1, 2ul}), 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 2), (dnac::ConstArrayView<float>{expectedSWValues2, 2ul}), 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 3), (dnac::ConstArrayView<float>{expectedSWValues3, 1ul}), 1ul, 0.0001f);
}
