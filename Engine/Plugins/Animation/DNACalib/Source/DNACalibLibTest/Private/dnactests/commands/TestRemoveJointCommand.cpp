// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/JointDNAReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"

#include <cstdint>
#include <numeric>
#include <vector>

namespace {

class RemoveJointCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        JointDNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;
};

}  // namespace

TEST_F(RemoveJointCommandTest, RemoveSingleJoint) {
    dnac::RemoveJointCommand cmd(1u);
    cmd.run(output.get());

    ASSERT_EQ(output->getJointCount(), 3u);
    ASSERT_EQ(output->getJointRowCount(), 3u * 9u);

    static const char* expectedJointNames[] = {"JA", "JC", "JD"};
    for (std::uint16_t i = 0; i < 3; ++i) {
        ASSERT_EQ(output->getJointName(i), (dnac::StringView{expectedJointNames[i], 2ul}));
    }

    static const std::uint16_t expectedJointIndicesPerLOD0[] = {0u, 1u};
    static const std::uint16_t expectedJointIndicesPerLOD1[] = {0u};
    ASSERT_ELEMENTS_EQ(output->getJointIndicesForLOD(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndicesPerLOD0, 2ul}),
                       2ul);
    ASSERT_ELEMENTS_EQ(output->getJointIndicesForLOD(1),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndicesPerLOD1, 1ul}),
                       1ul);

    ASSERT_EQ(output->getJointParentIndex(0), 0);
    ASSERT_EQ(output->getJointParentIndex(1), 0);
    ASSERT_EQ(output->getJointParentIndex(2), 1);
    ASSERT_EQ(output->getJointParentIndex(3), static_cast<std::uint16_t>(-1));

    static const float expectedNeutralXs[] = {1.0f, 7.0f, 10.0f};
    static const float expectedNeutralYs[] = {2.0f, 8.0f, 11.0f};
    static const float expectedNeutralZs[] = {3.0f, 9.0f, 12.0f};
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationXs(), (dnac::ConstArrayView<float>{expectedNeutralXs, 3ul}), 3ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationYs(), (dnac::ConstArrayView<float>{expectedNeutralYs, 3ul}), 3ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationZs(), (dnac::ConstArrayView<float>{expectedNeutralZs, 3ul}), 3ul);

    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationXs(), (dnac::ConstArrayView<float>{expectedNeutralXs, 3ul}), 3ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationYs(), (dnac::ConstArrayView<float>{expectedNeutralYs, 3ul}), 3ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationZs(), (dnac::ConstArrayView<float>{expectedNeutralZs, 3ul}), 3ul);

    ASSERT_EQ(output->getJointGroupCount(), 1u);

    static const std::uint16_t expectedJointIndices[] = {0u, 1u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupJointIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndices, 2ul}),
                       2ul);

    static const std::uint16_t expectedLODs[] = {2u, 1u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupLODs(0), (dnac::ConstArrayView<std::uint16_t>{expectedLODs, 2ul}), 2ul);

    static const std::uint16_t expectedInputIndices[] = {13, 56, 120};
    ASSERT_ELEMENTS_EQ(output->getJointGroupInputIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedInputIndices, 3ul}),
                       3ul);

    static const std::uint16_t expectedOutputIndices[] = {8, 9};
    ASSERT_ELEMENTS_EQ(output->getJointGroupOutputIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedOutputIndices, 2ul}),
                       2ul);

    static const float expectedValues[] = {0.5f, 0.2f, 0.3f, 0.1f, 0.75f, 1.0f};
    ASSERT_ELEMENTS_EQ(output->getJointGroupValues(0), (dnac::ConstArrayView<float>{expectedValues, 6ul}), 6ul);

    ASSERT_EQ(output->getSkinWeightsCount(0), 4u);

    static const std::uint16_t expectedSWJointIndices0[] = {0, 1};
    static const std::uint16_t expectedSWJointIndices1[] = {0};
    static const std::uint16_t expectedSWJointIndices2[] = {1};
    static const std::uint16_t expectedSWJointIndices3[] = {0};

    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices0, 2ul}),
                       2ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 1),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices1, 1ul}),
                       1ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 2),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices2, 1ul}),
                       1ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 3),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices3, 1ul}),
                       1ul);

    static const float expectedSWValues0[] = {0.3333f, 0.6666f};
    static const float expectedSWValues1[] = {1.0f};
    static const float expectedSWValues2[] = {1.0f};
    static const float expectedSWValues3[] = {1.0f};

    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 0), (dnac::ConstArrayView<float>{expectedSWValues0, 2ul}), 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 1), (dnac::ConstArrayView<float>{expectedSWValues1, 1ul}), 1ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 2), (dnac::ConstArrayView<float>{expectedSWValues2, 1ul}), 1ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 3), (dnac::ConstArrayView<float>{expectedSWValues3, 1ul}), 1ul, 0.0001f);
}

TEST_F(RemoveJointCommandTest, RemoveMultipleJoints) {
    std::vector<std::uint16_t> jointIndices{1u, 2u};
    dnac::RemoveJointCommand cmd(dnac::ConstArrayView<std::uint16_t>{jointIndices});
    cmd.run(output.get());

    ASSERT_EQ(output->getJointCount(), 2u);
    ASSERT_EQ(output->getJointRowCount(), 2u * 9u);

    static const char* expectedJointNames[] = {"JA", "JD"};
    for (std::uint16_t i = 0; i < 2; ++i) {
        ASSERT_EQ(output->getJointName(i), (dnac::StringView{expectedJointNames[i], 2ul}));
    }

    static const std::uint16_t expectedJointIndicesPerLOD0[] = {0u};
    static const std::uint16_t expectedJointIndicesPerLOD1[] = {0u};
    ASSERT_ELEMENTS_EQ(output->getJointIndicesForLOD(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndicesPerLOD0, 1ul}),
                       1ul);
    ASSERT_ELEMENTS_EQ(output->getJointIndicesForLOD(1),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndicesPerLOD1, 1ul}),
                       1ul);

    ASSERT_EQ(output->getJointParentIndex(0), 0);
    ASSERT_EQ(output->getJointParentIndex(1), 0);
    ASSERT_EQ(output->getJointParentIndex(2), static_cast<std::uint16_t>(-1));
    ASSERT_EQ(output->getJointParentIndex(3), static_cast<std::uint16_t>(-1));

    static const float expectedNeutralXs[] = {1.0f, 10.0f};
    static const float expectedNeutralYs[] = {2.0f, 11.0f};
    static const float expectedNeutralZs[] = {3.0f, 12.0f};
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationXs(), (dnac::ConstArrayView<float>{expectedNeutralXs, 2ul}), 2ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationYs(), (dnac::ConstArrayView<float>{expectedNeutralYs, 2ul}), 2ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationZs(), (dnac::ConstArrayView<float>{expectedNeutralZs, 2ul}), 2ul);

    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationXs(), (dnac::ConstArrayView<float>{expectedNeutralXs, 2ul}), 2ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationYs(), (dnac::ConstArrayView<float>{expectedNeutralYs, 2ul}), 2ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationZs(), (dnac::ConstArrayView<float>{expectedNeutralZs, 2ul}), 2ul);

    ASSERT_EQ(output->getJointGroupCount(), 1u);

    static const std::uint16_t expectedJointIndices[] = {0u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupJointIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndices, 1ul}),
                       1ul);

    static const std::uint16_t expectedLODs[] = {1u, 1u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupLODs(0), (dnac::ConstArrayView<std::uint16_t>{expectedLODs, 2ul}), 2ul);

    static const std::uint16_t expectedInputIndices[] = {13, 56, 120};
    ASSERT_ELEMENTS_EQ(output->getJointGroupInputIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedInputIndices, 3ul}),
                       3ul);

    static const std::uint16_t expectedOutputIndices[] = {8};
    ASSERT_ELEMENTS_EQ(output->getJointGroupOutputIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedOutputIndices, 1u}),
                       1u);

    static const float expectedValues[] = {0.5f, 0.2f, 0.3f};
    ASSERT_ELEMENTS_EQ(output->getJointGroupValues(0), (dnac::ConstArrayView<float>{expectedValues, 3u}), 3u);

    ASSERT_EQ(output->getSkinWeightsCount(0), 4u);

    static const std::uint16_t expectedSWJointIndices[] = {0};

    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices, 1ul}),
                       1ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 1),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices, 1ul}),
                       1ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 2),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices, 1ul}),
                       1ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 3),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices, 1ul}),
                       1ul);

    static const float expectedSWValues[] = {1.0f};

    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 0), (dnac::ConstArrayView<float>{expectedSWValues, 1ul}), 1ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 1), (dnac::ConstArrayView<float>{expectedSWValues, 1ul}), 1ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 2), (dnac::ConstArrayView<float>{expectedSWValues, 1ul}), 1ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 3), (dnac::ConstArrayView<float>{expectedSWValues, 1ul}), 1ul, 0.0001f);
}

TEST_F(RemoveJointCommandTest, RemoveAllJointsOneByOne) {
    const auto jointCount = output->getJointCount();
    dnac::RemoveJointCommand cmd;
    for (std::uint16_t jointIndex = 0u; jointIndex < jointCount; ++jointIndex) {
        cmd.setJointIndex(0);  // Due to remapping, 0, 1, 2 wouldn't remove all, as after removing 0th, 2nd would become 1st
        cmd.run(output.get());
    }

    ASSERT_EQ(output->getJointCount(), 0u);
    ASSERT_EQ(output->getJointRowCount(), 0u);

    ASSERT_EQ(output->getJointIndicesForLOD(0), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getJointIndicesForLOD(1), (dnac::ConstArrayView<std::uint16_t>{}));

    ASSERT_EQ(output->getJointParentIndex(0), static_cast<std::uint16_t>(-1));

    ASSERT_EQ(output->getNeutralJointTranslationXs(), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getNeutralJointTranslationYs(), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getNeutralJointTranslationZs(), (dnac::ConstArrayView<float>{}));

    ASSERT_EQ(output->getNeutralJointRotationXs(), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getNeutralJointRotationYs(), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getNeutralJointRotationZs(), (dnac::ConstArrayView<float>{}));

    ASSERT_EQ(output->getJointGroupCount(), 1u);

    ASSERT_EQ(output->getJointGroupJointIndices(0), (dnac::ConstArrayView<std::uint16_t>{}));

    static const std::uint16_t expectedLODs[] = {0u, 0u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupLODs(0), (dnac::ConstArrayView<std::uint16_t>{expectedLODs, 2ul}), 2ul);

    ASSERT_EQ(output->getJointGroupInputIndices(0), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getJointGroupOutputIndices(0), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getJointGroupValues(0), (dnac::ConstArrayView<float>{}));

    ASSERT_EQ(output->getSkinWeightsCount(0), 4u);

    ASSERT_EQ(output->getSkinWeightsJointIndices(0, 0), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getSkinWeightsJointIndices(0, 1), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getSkinWeightsJointIndices(0, 2), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getSkinWeightsJointIndices(0, 3), (dnac::ConstArrayView<std::uint16_t>{}));

    ASSERT_EQ(output->getSkinWeightsValues(0, 0), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getSkinWeightsValues(0, 1), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getSkinWeightsValues(0, 2), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getSkinWeightsValues(0, 3), (dnac::ConstArrayView<float>{}));
}

TEST_F(RemoveJointCommandTest, RemoveAllJoints) {
    const auto jointCount = output->getJointCount();
    dnac::RemoveJointCommand cmd;
    std::vector<std::uint16_t> jointsToRemove;
    jointsToRemove.resize(jointCount);
    std::iota(jointsToRemove.begin(), jointsToRemove.end(), static_cast<std::uint16_t>(0u));
    cmd.setJointIndices(dnac::ConstArrayView<std::uint16_t>{jointsToRemove});
    cmd.run(output.get());

    ASSERT_EQ(output->getJointCount(), 0u);
    ASSERT_EQ(output->getJointRowCount(), 0u);

    ASSERT_EQ(output->getJointIndicesForLOD(0), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getJointIndicesForLOD(1), (dnac::ConstArrayView<std::uint16_t>{}));

    ASSERT_EQ(output->getJointParentIndex(0), static_cast<std::uint16_t>(-1));

    ASSERT_EQ(output->getNeutralJointTranslationXs(), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getNeutralJointTranslationYs(), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getNeutralJointTranslationZs(), (dnac::ConstArrayView<float>{}));

    ASSERT_EQ(output->getNeutralJointRotationXs(), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getNeutralJointRotationYs(), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getNeutralJointRotationZs(), (dnac::ConstArrayView<float>{}));

    ASSERT_EQ(output->getJointGroupCount(), 1u);

    ASSERT_EQ(output->getJointGroupJointIndices(0), (dnac::ConstArrayView<std::uint16_t>{}));

    static const std::uint16_t expectedLODs[] = {0u, 0u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupLODs(0), (dnac::ConstArrayView<std::uint16_t>{expectedLODs, 2ul}), 2ul);

    ASSERT_EQ(output->getJointGroupInputIndices(0), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getJointGroupOutputIndices(0), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getJointGroupValues(0), (dnac::ConstArrayView<float>{}));

    ASSERT_EQ(output->getSkinWeightsCount(0), 4u);

    ASSERT_EQ(output->getSkinWeightsJointIndices(0, 0), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getSkinWeightsJointIndices(0, 1), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getSkinWeightsJointIndices(0, 2), (dnac::ConstArrayView<std::uint16_t>{}));
    ASSERT_EQ(output->getSkinWeightsJointIndices(0, 3), (dnac::ConstArrayView<std::uint16_t>{}));

    ASSERT_EQ(output->getSkinWeightsValues(0, 0), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getSkinWeightsValues(0, 1), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getSkinWeightsValues(0, 2), (dnac::ConstArrayView<float>{}));
    ASSERT_EQ(output->getSkinWeightsValues(0, 3), (dnac::ConstArrayView<float>{}));
}

TEST_F(RemoveJointCommandTest, RemoveJointNotInAnyLOD) {
    dnac::RemoveJointCommand cmd(3u);
    cmd.run(output.get());

    ASSERT_EQ(output->getJointCount(), 3u);
    ASSERT_EQ(output->getJointRowCount(), 3u * 9u);

    static const char* expectedJointNames[] = {"JA", "JB", "JC"};
    for (std::uint16_t i = 0; i < 3; ++i) {
        ASSERT_EQ(output->getJointName(i), (dnac::StringView{expectedJointNames[i], 2ul}));
    }

    static const std::uint16_t expectedJointIndicesPerLOD0[] = {0u, 1u, 2u};
    static const std::uint16_t expectedJointIndicesPerLOD1[] = {0u, 1u};
    ASSERT_ELEMENTS_EQ(output->getJointIndicesForLOD(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndicesPerLOD0, 3ul}),
                       3ul);
    ASSERT_ELEMENTS_EQ(output->getJointIndicesForLOD(1),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndicesPerLOD1, 2ul}),
                       2ul);

    ASSERT_EQ(output->getJointParentIndex(0), 0);
    ASSERT_EQ(output->getJointParentIndex(1), 0);
    ASSERT_EQ(output->getJointParentIndex(2), 1);
    ASSERT_EQ(output->getJointParentIndex(3), static_cast<std::uint16_t>(-1));

    static const float expectedNeutralXs[] = {1.0f, 4.0f, 7.0f};
    static const float expectedNeutralYs[] = {2.0f, 5.0f, 8.0f};
    static const float expectedNeutralZs[] = {3.0f, 6.0f, 9.0f};
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationXs(), (dnac::ConstArrayView<float>{expectedNeutralXs, 3ul}), 3ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationYs(), (dnac::ConstArrayView<float>{expectedNeutralYs, 3ul}), 3ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointTranslationZs(), (dnac::ConstArrayView<float>{expectedNeutralZs, 3ul}), 3ul);

    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationXs(), (dnac::ConstArrayView<float>{expectedNeutralXs, 3ul}), 3ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationYs(), (dnac::ConstArrayView<float>{expectedNeutralYs, 3ul}), 3ul);
    ASSERT_ELEMENTS_EQ(output->getNeutralJointRotationZs(), (dnac::ConstArrayView<float>{expectedNeutralZs, 3ul}), 3ul);

    ASSERT_EQ(output->getJointGroupCount(), 1u);

    static const std::uint16_t expectedJointIndices[] = {0u, 1u, 2u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupJointIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedJointIndices, 3ul}),
                       3ul);

    static const std::uint16_t expectedLODs[] = {4u, 2u};
    ASSERT_ELEMENTS_EQ(output->getJointGroupLODs(0), (dnac::ConstArrayView<std::uint16_t>{expectedLODs, 2ul}), 2ul);

    static const std::uint16_t expectedInputIndices[] = {13, 56, 120};
    ASSERT_ELEMENTS_EQ(output->getJointGroupInputIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedInputIndices, 3ul}),
                       3ul);

    static const std::uint16_t expectedOutputIndices[] = {8, 9, 17, 18};
    ASSERT_ELEMENTS_EQ(output->getJointGroupOutputIndices(0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedOutputIndices, 4ul}),
                       4ul);

    static const float expectedValues[] = {0.5f, 0.2f, 0.3f, 0.25f, 0.4f, 0.15f, 0.1f, 0.1f, 0.9f, 0.1f, 0.75f, 1.0f};
    ASSERT_ELEMENTS_EQ(output->getJointGroupValues(0), (dnac::ConstArrayView<float>{expectedValues, 12ul}), 12ul);

    ASSERT_EQ(output->getSkinWeightsCount(0), 4u);

    static const std::uint16_t expectedSWJointIndices0[] = {0, 1, 2};
    static const std::uint16_t expectedSWJointIndices1[] = {0, 1};
    static const std::uint16_t expectedSWJointIndices2[] = {1, 2};
    static const std::uint16_t expectedSWJointIndices3[] = {1};

    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices0, 3ul}),
                       3ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 1),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices1, 2ul}),
                       2ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 2),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices2, 2ul}),
                       2ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 3),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices3, 1ul}),
                       1ul);

    static const float expectedSWValues0[] = {0.1f, 0.7f, 0.2f};
    static const float expectedSWValues1[] = {0.2f, 0.8f};
    static const float expectedSWValues2[] = {0.4f, 0.6f};
    static const float expectedSWValues3[] = {1.0f};

    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 0), (dnac::ConstArrayView<float>{expectedSWValues0, 3ul}), 3ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 1), (dnac::ConstArrayView<float>{expectedSWValues1, 2ul}), 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 2), (dnac::ConstArrayView<float>{expectedSWValues2, 2ul}), 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 3), (dnac::ConstArrayView<float>{expectedSWValues3, 1ul}), 1ul, 0.0001f);
}
