// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/MeshDNAReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"

#include <cstdint>
#include <numeric>
#include <vector>

namespace {

class RemoveMeshCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        MeshDNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;
};

}  // namespace

TEST_F(RemoveMeshCommandTest, RemoveSingleMesh) {
    dnac::RemoveMeshCommand cmd(static_cast<std::uint16_t>(0));

    ASSERT_EQ(output->getLODCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getMeshCount(), 3u);
    ASSERT_STREQ(output->getMeshName(0).c_str(), "mesh0");
    ASSERT_STREQ(output->getMeshName(1).c_str(), "mesh1");
    ASSERT_STREQ(output->getMeshName(2).c_str(), "mesh2");

    ASSERT_EQ(output->getMeshIndexListCount(), 2u);

    const std::array<std::uint16_t, 3u> meshIndicesForLOD0 = {0u, 1u, 2u};
    ASSERT_ELEMENTS_EQ(output->getMeshIndicesForLOD(0), meshIndicesForLOD0, meshIndicesForLOD0.size());
    const std::array<std::uint16_t, 2u> meshIndicesForLOD1 = {1u, 2u};
    ASSERT_ELEMENTS_EQ(output->getMeshIndicesForLOD(1), meshIndicesForLOD1, meshIndicesForLOD1.size());

    ASSERT_EQ(output->getMeshBlendShapeChannelMappingCount(), 6u);
    const auto mbscm0 = output->getMeshBlendShapeChannelMapping(0u);
    ASSERT_EQ(mbscm0.meshIndex, 0u);
    ASSERT_EQ(mbscm0.blendShapeChannelIndex, 0u);

    const auto mbscm1 = output->getMeshBlendShapeChannelMapping(1u);
    ASSERT_EQ(mbscm1.meshIndex, 0u);
    ASSERT_EQ(mbscm1.blendShapeChannelIndex, 1u);

    const auto mbscm2 = output->getMeshBlendShapeChannelMapping(2u);
    ASSERT_EQ(mbscm2.meshIndex, 1u);
    ASSERT_EQ(mbscm2.blendShapeChannelIndex, 2u);

    const auto mbscm3 = output->getMeshBlendShapeChannelMapping(3u);
    ASSERT_EQ(mbscm3.meshIndex, 1u);
    ASSERT_EQ(mbscm3.blendShapeChannelIndex, 3u);

    const auto mbscm4 = output->getMeshBlendShapeChannelMapping(4u);
    ASSERT_EQ(mbscm4.meshIndex, 2u);
    ASSERT_EQ(mbscm4.blendShapeChannelIndex, 4u);

    const auto mbscm5 = output->getMeshBlendShapeChannelMapping(5u);
    ASSERT_EQ(mbscm5.meshIndex, 2u);
    ASSERT_EQ(mbscm5.blendShapeChannelIndex, 5u);

    const std::array<std::uint16_t, 6u> meshBlendShapeChannelMappingIndicesForLOD0 = {0u, 1u, 2u, 3u, 4u, 5u};
    ASSERT_ELEMENTS_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(0u),
                       meshBlendShapeChannelMappingIndicesForLOD0,
                       meshBlendShapeChannelMappingIndicesForLOD0.size());

    const std::array<std::uint16_t, 4u> meshBlendShapeChannelMappingIndicesForLOD1 = {2u, 3u, 4u, 5u};
    ASSERT_ELEMENTS_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(1u),
                       meshBlendShapeChannelMappingIndicesForLOD1,
                       meshBlendShapeChannelMappingIndicesForLOD1.size());

    // Check geometry
    ASSERT_EQ(output->getVertexPositionCount(0u), 3u);
    ASSERT_EQ(output->getVertexPositionCount(1u), 2u);
    ASSERT_EQ(output->getVertexPositionCount(2u), 2u);

    // Remove mesh "mesh0"
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getMeshCount(), 2u);
    ASSERT_STREQ(output->getMeshName(0).c_str(), "mesh1");
    ASSERT_STREQ(output->getMeshName(1).c_str(), "mesh2");

    ASSERT_EQ(output->getMeshIndexListCount(), 2u);

    const std::array<std::uint16_t, 2u> filteredMeshIndicesForLOD0 = {0u, 1u};
    ASSERT_ELEMENTS_EQ(output->getMeshIndicesForLOD(0), filteredMeshIndicesForLOD0, filteredMeshIndicesForLOD0.size());
    const std::array<std::uint16_t, 2u> filteredMeshIndicesForLOD1 = {0u, 1u};
    ASSERT_ELEMENTS_EQ(output->getMeshIndicesForLOD(1), filteredMeshIndicesForLOD1, filteredMeshIndicesForLOD1.size());

    ASSERT_EQ(output->getMeshBlendShapeChannelMappingCount(), 4u);

    const auto fmbscm0 = output->getMeshBlendShapeChannelMapping(0u);
    ASSERT_EQ(fmbscm0.meshIndex, 0u);
    ASSERT_EQ(fmbscm0.blendShapeChannelIndex, 2u);

    const auto fmbscm1 = output->getMeshBlendShapeChannelMapping(1u);
    ASSERT_EQ(fmbscm1.meshIndex, 0u);
    ASSERT_EQ(fmbscm1.blendShapeChannelIndex, 3u);

    const auto fmbscm2 = output->getMeshBlendShapeChannelMapping(2u);
    ASSERT_EQ(fmbscm2.meshIndex, 1u);
    ASSERT_EQ(fmbscm2.blendShapeChannelIndex, 4u);

    const auto fmbscm3 = output->getMeshBlendShapeChannelMapping(3u);
    ASSERT_EQ(fmbscm3.meshIndex, 1u);
    ASSERT_EQ(fmbscm3.blendShapeChannelIndex, 5u);

    const std::array<std::uint16_t, 4u> filteredMeshBlendShapeChannelMappingIndicesForLOD0 = {0u, 1u, 2u, 3u};
    ASSERT_ELEMENTS_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(0u),
                       filteredMeshBlendShapeChannelMappingIndicesForLOD0,
                       filteredMeshBlendShapeChannelMappingIndicesForLOD0.size());

    const std::array<std::uint16_t, 4u> filteredMeshBlendShapeChannelMappingIndicesForLOD1 = {0u, 1u, 2u, 3u};
    ASSERT_ELEMENTS_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(1u),
                       filteredMeshBlendShapeChannelMappingIndicesForLOD1,
                       filteredMeshBlendShapeChannelMappingIndicesForLOD1.size());

    // Check geometry
    ASSERT_EQ(output->getVertexPositionCount(0u), 2u);
    ASSERT_EQ(output->getVertexPositionCount(1u), 2u);
    ASSERT_EQ(output->getVertexPositionCount(2u), 0u);
}

TEST_F(RemoveMeshCommandTest, RemoveMultipleMeshes) {
    std::vector<std::uint16_t> meshIndices{0u, 2u};
    dnac::RemoveMeshCommand cmd;
    cmd.setMeshIndices(dnac::ConstArrayView<std::uint16_t>{meshIndices});

    ASSERT_EQ(output->getLODCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getMeshCount(), 3u);
    ASSERT_STREQ(output->getMeshName(0).c_str(), "mesh0");
    ASSERT_STREQ(output->getMeshName(1).c_str(), "mesh1");
    ASSERT_STREQ(output->getMeshName(2).c_str(), "mesh2");

    ASSERT_EQ(output->getMeshIndexListCount(), 2u);

    const std::array<std::uint16_t, 3u> meshIndicesForLOD0 = {0u, 1u, 2u};
    ASSERT_ELEMENTS_EQ(output->getMeshIndicesForLOD(0), meshIndicesForLOD0, meshIndicesForLOD0.size());
    const std::array<std::uint16_t, 2u> meshIndicesForLOD1 = {1u, 2u};
    ASSERT_ELEMENTS_EQ(output->getMeshIndicesForLOD(1), meshIndicesForLOD1, meshIndicesForLOD1.size());

    ASSERT_EQ(output->getMeshBlendShapeChannelMappingCount(), 6u);
    const auto mbscm0 = output->getMeshBlendShapeChannelMapping(0u);
    ASSERT_EQ(mbscm0.meshIndex, 0u);
    ASSERT_EQ(mbscm0.blendShapeChannelIndex, 0u);

    const auto mbscm1 = output->getMeshBlendShapeChannelMapping(1u);
    ASSERT_EQ(mbscm1.meshIndex, 0u);
    ASSERT_EQ(mbscm1.blendShapeChannelIndex, 1u);

    const auto mbscm2 = output->getMeshBlendShapeChannelMapping(2u);
    ASSERT_EQ(mbscm2.meshIndex, 1u);
    ASSERT_EQ(mbscm2.blendShapeChannelIndex, 2u);

    const auto mbscm3 = output->getMeshBlendShapeChannelMapping(3u);
    ASSERT_EQ(mbscm3.meshIndex, 1u);
    ASSERT_EQ(mbscm3.blendShapeChannelIndex, 3u);

    const auto mbscm4 = output->getMeshBlendShapeChannelMapping(4u);
    ASSERT_EQ(mbscm4.meshIndex, 2u);
    ASSERT_EQ(mbscm4.blendShapeChannelIndex, 4u);

    const auto mbscm5 = output->getMeshBlendShapeChannelMapping(5u);
    ASSERT_EQ(mbscm5.meshIndex, 2u);
    ASSERT_EQ(mbscm5.blendShapeChannelIndex, 5u);

    const std::array<std::uint16_t, 6u> meshBlendShapeChannelMappingIndicesForLOD0 = {0u, 1u, 2u, 3u, 4u, 5u};
    ASSERT_ELEMENTS_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(0u),
                       meshBlendShapeChannelMappingIndicesForLOD0,
                       meshBlendShapeChannelMappingIndicesForLOD0.size());

    const std::array<std::uint16_t, 4u> meshBlendShapeChannelMappingIndicesForLOD1 = {2u, 3u, 4u, 5u};
    ASSERT_ELEMENTS_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(1u),
                       meshBlendShapeChannelMappingIndicesForLOD1,
                       meshBlendShapeChannelMappingIndicesForLOD1.size());

    // Check geometry
    ASSERT_EQ(output->getVertexPositionCount(0u), 3u);
    ASSERT_EQ(output->getVertexPositionCount(1u), 2u);
    ASSERT_EQ(output->getVertexPositionCount(2u), 2u);

    // Remove meshes "mesh0" and "mesh2"
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getMeshCount(), 1u);
    ASSERT_STREQ(output->getMeshName(0).c_str(), "mesh1");

    ASSERT_EQ(output->getMeshIndexListCount(), 2u);

    const std::array<std::uint16_t, 1u> filteredMeshIndicesForLOD0 = {0u};
    ASSERT_ELEMENTS_EQ(output->getMeshIndicesForLOD(0), filteredMeshIndicesForLOD0, filteredMeshIndicesForLOD0.size());
    const std::array<std::uint16_t, 1u> filteredMeshIndicesForLOD1 = {0u};
    ASSERT_ELEMENTS_EQ(output->getMeshIndicesForLOD(1), filteredMeshIndicesForLOD1, filteredMeshIndicesForLOD1.size());

    ASSERT_EQ(output->getMeshBlendShapeChannelMappingCount(), 2u);

    const auto fmbscm0 = output->getMeshBlendShapeChannelMapping(0u);
    ASSERT_EQ(fmbscm0.meshIndex, 0u);
    ASSERT_EQ(fmbscm0.blendShapeChannelIndex, 2u);

    const auto fmbscm1 = output->getMeshBlendShapeChannelMapping(1u);
    ASSERT_EQ(fmbscm1.meshIndex, 0u);
    ASSERT_EQ(fmbscm1.blendShapeChannelIndex, 3u);

    const std::array<std::uint16_t, 2u> filteredMeshBlendShapeChannelMappingIndicesForLOD0 = {0u, 1u};
    ASSERT_ELEMENTS_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(0u),
                       filteredMeshBlendShapeChannelMappingIndicesForLOD0,
                       filteredMeshBlendShapeChannelMappingIndicesForLOD0.size());

    const std::array<std::uint16_t, 2u> filteredMeshBlendShapeChannelMappingIndicesForLOD1 = {0u, 1u};
    ASSERT_ELEMENTS_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(1u),
                       filteredMeshBlendShapeChannelMappingIndicesForLOD1,
                       filteredMeshBlendShapeChannelMappingIndicesForLOD1.size());

    // Check geometry
    ASSERT_EQ(output->getVertexPositionCount(0u), 2u);
    ASSERT_EQ(output->getVertexPositionCount(1u), 0u);
}

TEST_F(RemoveMeshCommandTest, RemoveAllMeshesOneByOne) {
    const auto meshCount = output->getMeshCount();
    dnac::RemoveMeshCommand cmd;
    for (std::uint16_t meshIndex = 0u; meshIndex < meshCount; ++meshIndex) {
        cmd.setMeshIndex(0);  // Due to remapping, 0, 1, 2 wouldn't remove all, as after removing 0th, 2nd would become 1st
        cmd.run(output.get());
    }

    ASSERT_EQ(output->getLODCount(), 2u);

    // Check definition
    ASSERT_EQ(output->getMeshCount(), 0u);

    ASSERT_EQ(output->getMeshIndexListCount(), 2u);
    ASSERT_EQ(output->getMeshIndicesForLOD(0).size(), 0ul);
    ASSERT_EQ(output->getMeshIndicesForLOD(1).size(), 0ul);

    ASSERT_EQ(output->getMeshBlendShapeChannelMappingCount(), 0u);
    ASSERT_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(0u).size(), 0ul);
    ASSERT_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(1u).size(), 0ul);

    // Check geometry
    ASSERT_EQ(output->getVertexPositionCount(0u), 0u);
}

TEST_F(RemoveMeshCommandTest, RemoveAllMeshes) {
    const auto meshCount = output->getMeshCount();
    std::vector<std::uint16_t> meshesToRemove;
    meshesToRemove.resize(meshCount);
    std::iota(meshesToRemove.begin(), meshesToRemove.end(), static_cast<std::uint16_t>(0u));
    dnac::RemoveMeshCommand cmd{dnac::ConstArrayView<std::uint16_t>{meshesToRemove}};
    cmd.run(output.get());

    // Check definition
    ASSERT_EQ(output->getMeshCount(), 0u);

    ASSERT_EQ(output->getMeshIndexListCount(), 2u);
    ASSERT_EQ(output->getMeshIndicesForLOD(0).size(), 0ul);
    ASSERT_EQ(output->getMeshIndicesForLOD(1).size(), 0ul);

    ASSERT_EQ(output->getMeshBlendShapeChannelMappingCount(), 0u);
    ASSERT_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(0u).size(), 0ul);
    ASSERT_EQ(output->getMeshBlendShapeChannelMappingIndicesForLOD(1u).size(), 0ul);

    // Check geometry
    ASSERT_EQ(output->getVertexPositionCount(0u), 0u);
}
