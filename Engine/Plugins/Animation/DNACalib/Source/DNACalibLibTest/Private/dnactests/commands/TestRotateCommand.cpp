// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"

#include <array>
#include <cstdint>

namespace {

class RotatableDNAReader : public dna::FakeDNACReader {
public:
    explicit RotatableDNAReader(dnac::MemoryResource* memRes = nullptr) :
        neutralJointRotations{memRes},
        neutralJointTranslations{memRes},
        vertexPositions{memRes},
        blendShapeNames{memRes},
        bsChannelIndices{memRes},
        bsTargetDeltas{memRes},
        bsTargetVertexIndices{memRes} {

        jointHierarchy = {0, 0};
        float jrxs[] = {1.0f, 2.5f};
        float jrys[] = {3.0f, 4.5f};
        float jrzs[] = {4.0f, 8.0f};
        neutralJointRotations.xs.assign(jrxs, jrxs + 2ul);
        neutralJointRotations.ys.assign(jrys, jrys + 2ul);
        neutralJointRotations.zs.assign(jrzs, jrzs + 2ul);

        float jtxs[] = {1.0f, 2.5f};
        float jtys[] = {3.0f, 4.5f};
        float jtzs[] = {4.0f, 8.0f};
        neutralJointTranslations.xs.assign(jtxs, jtxs + 2ul);
        neutralJointTranslations.ys.assign(jtys, jtys + 2ul);
        neutralJointTranslations.zs.assign(jtzs, jtzs + 2ul);

        float vxs[] = {4.0f, 12.0f, 23.5f, -4.0f, 2.0f};
        float vys[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f};
        float vzs[] = {11.0f, -5.5f, 22.0f, 3.0f, 6.1f};
        vertexPositions.xs.assign(vxs, vxs + 5ul);
        vertexPositions.ys.assign(vys, vys + 5ul);
        vertexPositions.zs.assign(vzs, vzs + 5ul);

        bsChannelIndices.assign({{0u, 1u, 2u}});

        const auto meshCount = RotatableDNAReader::getMeshCount();
        bsTargetDeltas.resize(meshCount);
        bsTargetDeltas[0].resize(3u, dnac::RawVector3Vector{memRes});
        float xs1[] = {1.0f, 1.0f, 1.0f, 1.0f, 1.0f};
        float ys1[] = {2.0f, 2.0f, 2.0f, 2.0f, 2.0f};
        float zs1[] = {3.0f, 3.0f, 3.0f, 3.0f, 3.0f};

        float xs2[] = {4.0f, 4.0f, 4.0f, 4.0f, 4.0f};
        float ys2[] = {5.0f, 5.0f, 5.0f, 5.0f, 5.0f};
        float zs2[] = {6.0f, 6.0f, 6.0f, 6.0f, 6.0f};

        float xs3[] = {7.0f, 7.0f, 7.0f, 7.0f, 7.0f};
        float ys3[] = {8.0f, 8.0f, 8.0f, 8.0f, 8.0f};
        float zs3[] = {9.0f, 9.0f, 9.0f, 9.0f, 9.0f};

        bsTargetDeltas[0][0].xs.assign(xs1, xs1 + 5u);
        bsTargetDeltas[0][0].ys.assign(ys1, ys1 + 5u);
        bsTargetDeltas[0][0].zs.assign(zs1, zs1 + 5u);

        bsTargetDeltas[0][1].xs.assign(xs2, xs2 + 5u);
        bsTargetDeltas[0][1].ys.assign(ys2, ys2 + 5u);
        bsTargetDeltas[0][1].zs.assign(zs2, zs2 + 5u);

        bsTargetDeltas[0][2].xs.assign(xs3, xs3 + 5u);
        bsTargetDeltas[0][2].ys.assign(ys3, ys3 + 5u);
        bsTargetDeltas[0][2].zs.assign(zs3, zs3 + 5u);

        bsTargetVertexIndices.resize(meshCount);
        bsTargetVertexIndices[0].resize(3u);
        bsTargetVertexIndices[0][0].assign({0u, 1u, 2u, 3u, 4u});
        bsTargetVertexIndices[0][1].assign({0u, 1u, 2u, 3u, 4u});
        bsTargetVertexIndices[0][2].assign({0u, 1u, 2u, 3u, 4u});
    }

    std::uint16_t getJointParentIndex(std::uint16_t index) const override {
        return jointHierarchy[index];
    }

    std::uint16_t getJointCount() const override {
        return static_cast<std::uint16_t>(jointHierarchy.size());
    }

    dnac::StringView getJointName(std::uint16_t /*unused*/) const override {
        return dnac::StringView{"A", 1ul};
    }

    std::uint16_t getMeshCount() const override {
        return 1;
    }

    dnac::StringView getMeshName(std::uint16_t /*unused*/) const override {
        return dnac::StringView{"M", 1ul};
    }

    dnac::Vector3 getNeutralJointRotation(std::uint16_t index) const override {
        return dnac::Vector3{neutralJointRotations.xs[index], neutralJointRotations.ys[index], neutralJointRotations.zs[index]};
    }

    dnac::ConstArrayView<float> getNeutralJointRotationXs() const override {
        return dnac::ConstArrayView<float>{neutralJointRotations.xs};
    }

    dnac::ConstArrayView<float> getNeutralJointRotationYs() const override {
        return dnac::ConstArrayView<float>{neutralJointRotations.ys};
    }

    dnac::ConstArrayView<float> getNeutralJointRotationZs() const override {
        return dnac::ConstArrayView<float>{neutralJointRotations.zs};
    }

    dnac::Vector3 getNeutralJointTranslation(std::uint16_t index) const override {
        return dnac::Vector3{neutralJointTranslations.xs[index],
                             neutralJointTranslations.ys[index],
                             neutralJointTranslations.zs[index]};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationXs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.xs};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationYs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.ys};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationZs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.zs};
    }

    std::uint32_t getVertexPositionCount(std::uint16_t /*unused*/) const override {
        return static_cast<std::uint32_t>(vertexPositions.size());
    }

    dnac::Vector3 getVertexPosition(std::uint16_t /*unused*/, std::uint32_t vertexIndex) const override {
        return dnac::Vector3{vertexPositions.xs[vertexIndex], vertexPositions.ys[vertexIndex], vertexPositions.zs[vertexIndex]};
    }

    dnac::ConstArrayView<float> getVertexPositionXs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.xs};
    }

    dnac::ConstArrayView<float> getVertexPositionYs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.ys};
    }

    dnac::ConstArrayView<float> getVertexPositionZs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.zs};
    }

    std::uint16_t getBlendShapeChannelCount() const override {
        return static_cast<std::uint16_t>(blendShapeNames.size());
    }

    dnac::StringView getBlendShapeChannelName(std::uint16_t index) const override {
        return dnac::StringView{blendShapeNames[index]};
    }

    std::uint16_t getBlendShapeTargetCount(std::uint16_t meshIndex) const override {
        if (meshIndex < getMeshCount()) {
            return static_cast<std::uint16_t>(bsChannelIndices[meshIndex].size());
        }
        return {};
    }

    std::uint16_t getBlendShapeChannelIndex(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return bsChannelIndices[meshIndex][blendShapeTargetIndex];
            }
        }
        return {};
    }

    std::uint32_t getBlendShapeTargetDeltaCount(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return static_cast<std::uint32_t>(bsTargetDeltas[meshIndex][blendShapeTargetIndex].size());
            }
        }
        return {};
    }

    dnac::Vector3 getBlendShapeTargetDelta(std::uint16_t meshIndex,
                                           std::uint16_t blendShapeTargetIndex,
                                           std::uint32_t deltaIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                if (deltaIndex < getBlendShapeTargetDeltaCount(meshIndex, blendShapeTargetIndex)) {
                    return dnac::Vector3{bsTargetDeltas[meshIndex][blendShapeTargetIndex].xs[deltaIndex],
                                         bsTargetDeltas[meshIndex][blendShapeTargetIndex].ys[deltaIndex],
                                         bsTargetDeltas[meshIndex][blendShapeTargetIndex].zs[deltaIndex]};
                }
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].xs};
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].ys};
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].zs};
            }
        }
        return {};
    }

    dnac::ConstArrayView<std::uint32_t> getBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                                         std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<std::uint32_t>{bsTargetVertexIndices[meshIndex][blendShapeTargetIndex]};
            }
        }
        return {};
    }

private:
    std::array<std::uint16_t, 2> jointHierarchy;
    dnac::RawVector3Vector neutralJointRotations;
    dnac::RawVector3Vector neutralJointTranslations;
    dnac::RawVector3Vector vertexPositions;

    dnac::Vector<dnac::String<char>> blendShapeNames;
    dnac::Matrix<std::uint16_t> bsChannelIndices;
    dnac::Matrix<dnac::RawVector3Vector> bsTargetDeltas;
    dnac::Matrix<dnac::Vector<std::uint32_t>> bsTargetVertexIndices;
};

class RotateCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        RotatableDNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
        degrees = {0.0f, 0.0f, 94.0f};
        origin = {10.0f, 0.0f, 0.0f};

        float jtxs[] = {7.63512f, 2.5f};
        float jtys[] = {-9.18735f, 4.5f};
        float jtzs[] = {4.0f, 8.0f};
        expectedNeutralJointTranslationXs.assign(jtxs, jtxs + 2ul);
        expectedNeutralJointTranslationYs.assign(jtys, jtys + 2ul);
        expectedNeutralJointTranslationZs.assign(jtzs, jtzs + 2ul);

        float jrxs[] = {1.0f, 2.5f};
        float jrys[] = {3.0f, 4.5f};
        float jrzs[] = {98.0f, 8.0f};
        expectedNeutralJointRotationXs.assign(jrxs, jrxs + 2ul);
        expectedNeutralJointRotationYs.assign(jrys, jrys + 2ul);
        expectedNeutralJointRotationZs.assign(jrzs, jrzs + 2ul);

        float vxs[] = {9.42097f, 7.86536f, 6.0656f, 6.98633f, 5.57023f};
        float vys[] = {-6.05514f, 1.85561f, 13.2578f, -14.2449f, -8.3293f};
        float vzs[] = {11.0f, -5.5f, 22.0f, 3.0f, 6.1f};
        expectedVertexPositionXs.assign(vxs, vxs + 5ul);
        expectedVertexPositionYs.assign(vys, vys + 5ul);
        expectedVertexPositionZs.assign(vzs, vzs + 5ul);

        float dxs0[] = {8.632681f, 8.632681f, 8.632681f, 8.632681f, 8.632681f};
        float dys0[] = {-9.11759f, -9.11759f, -9.11759f, -9.11759f, -9.11759f};
        float dzs0[] = {3.0f, 3.0f, 3.0f, 3.0f, 3.0f};
        expectedDelta0Xs.assign(dxs0, dxs0 + 5ul);
        expectedDelta0Ys.assign(dys0, dys0 + 5ul);
        expectedDelta0Zs.assign(dzs0, dzs0 + 5ul);

        float dxs1[] = {5.430718f, 5.430718f, 5.430718f, 5.430718f, 5.430718f};
        float dys1[] = {-6.334167f, -6.334167f, -6.334167f, -6.334167f, -6.334167f};
        float dzs1[] = {6.0f, 6.0f, 6.0f, 6.0f, 6.0f};
        expectedDelta1Xs.assign(dxs1, dxs1 + 5ul);
        expectedDelta1Ys.assign(dys1, dys1 + 5ul);
        expectedDelta1Zs.assign(dzs1, dzs1 + 5ul);

        float dxs2[] = {2.228757f, 2.228757f, 2.228757f, 2.228757f, 2.228757f};
        float dys2[] = {-3.550745f, -3.550745f, -3.550745f, -3.550745f, -3.550745f};
        float dzs2[] = {9.0f, 9.0f, 9.0f, 9.0f, 9.0f};
        expectedDelta2Xs.assign(dxs2, dxs2 + 5ul);
        expectedDelta2Ys.assign(dys2, dys2 + 5ul);
        expectedDelta2Zs.assign(dzs2, dzs2 + 5ul);
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;

    dnac::Vector3 degrees;
    dnac::Vector3 origin;

    dnac::Vector<float> expectedNeutralJointRotationXs;
    dnac::Vector<float> expectedNeutralJointRotationYs;
    dnac::Vector<float> expectedNeutralJointRotationZs;
    dnac::Vector<float> expectedNeutralJointTranslationXs;
    dnac::Vector<float> expectedNeutralJointTranslationYs;
    dnac::Vector<float> expectedNeutralJointTranslationZs;
    dnac::Vector<float> expectedVertexPositionXs;
    dnac::Vector<float> expectedVertexPositionYs;
    dnac::Vector<float> expectedVertexPositionZs;
    dnac::Vector<float> expectedDelta0Xs;
    dnac::Vector<float> expectedDelta0Ys;
    dnac::Vector<float> expectedDelta0Zs;
    dnac::Vector<float> expectedDelta1Xs;
    dnac::Vector<float> expectedDelta1Ys;
    dnac::Vector<float> expectedDelta1Zs;
    dnac::Vector<float> expectedDelta2Xs;
    dnac::Vector<float> expectedDelta2Ys;
    dnac::Vector<float> expectedDelta2Zs;
};

}  // namespace

TEST_F(RotateCommandTest, AlongZAxis) {
    dnac::RotateCommand cmd(degrees, origin);
    cmd.run(output.get());

    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationXs(),
                         expectedNeutralJointTranslationXs,
                         expectedNeutralJointTranslationXs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationYs(),
                         expectedNeutralJointTranslationYs,
                         expectedNeutralJointTranslationYs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationZs(),
                         expectedNeutralJointTranslationZs,
                         expectedNeutralJointTranslationZs.size(),
                         0.0001);

    ASSERT_ELEMENTS_NEAR(output->getNeutralJointRotationXs(),
                         expectedNeutralJointRotationXs,
                         expectedNeutralJointRotationXs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointRotationYs(),
                         expectedNeutralJointRotationYs,
                         expectedNeutralJointRotationYs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointRotationZs(),
                         expectedNeutralJointRotationZs,
                         expectedNeutralJointRotationZs.size(),
                         0.0001);

    ASSERT_ELEMENTS_NEAR(output->getVertexPositionXs(0), expectedVertexPositionXs, expectedVertexPositionXs.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionYs(0), expectedVertexPositionYs, expectedVertexPositionYs.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionZs(0), expectedVertexPositionZs, expectedVertexPositionZs.size(), 0.0001);

    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(0, 0).size(), expectedDelta0Xs.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(0, 0).size(), expectedDelta0Ys.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(0, 0).size(), expectedDelta0Zs.size());
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(0, 0), expectedDelta0Xs, expectedDelta0Xs.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(0, 0), expectedDelta0Ys, expectedDelta0Ys.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(0, 0), expectedDelta0Zs, expectedDelta0Zs.size(), 0.0001);
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(0, 1).size(), expectedDelta1Xs.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(0, 1).size(), expectedDelta1Ys.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(0, 1).size(), expectedDelta1Zs.size());
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(0, 1), expectedDelta1Xs, expectedDelta1Xs.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(0, 1), expectedDelta1Ys, expectedDelta1Ys.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(0, 1), expectedDelta1Zs, expectedDelta1Zs.size(), 0.0001);
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(0, 2).size(), expectedDelta2Xs.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(0, 2).size(), expectedDelta2Ys.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(0, 2).size(), expectedDelta2Zs.size());
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(0, 2), expectedDelta2Xs, expectedDelta2Xs.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(0, 2), expectedDelta2Ys, expectedDelta2Ys.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(0, 2), expectedDelta2Zs, expectedDelta2Zs.size(), 0.0001);
}
