// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"

#include <array>
#include <cstdint>

namespace {

class ScalableDNAReader : public dna::FakeDNACReader {
public:
    explicit ScalableDNAReader(dnac::MemoryResource* memRes = nullptr) :
        neutralJointTranslations{memRes},
        jointGroupInputIndices{memRes},
        jointGroupOutputIndices{memRes},
        jointGroupValues{memRes},
        vertexPositions{memRes},
        blendShapeTargetDeltas{memRes} {

        jointGroupCount = 1u;
        jointHierarchy = {0, 0};
        float jxs[] = {1.0f, 2.5f};
        float jys[] = {3.0f, 4.5f};
        float jzs[] = {4.0f, 8.0f};
        neutralJointTranslations.xs.assign(jxs, jxs + 2ul);
        neutralJointTranslations.ys.assign(jys, jys + 2ul);
        neutralJointTranslations.zs.assign(jzs, jzs + 2ul);

        std::uint16_t inputIndices[] = {0, 1, 2};
        std::uint16_t outputIndices[] = {0, 1, 3, 9};
        float values[] = {0.5f, 0.2f, 0.3f, 0.25f, 0.4f, 0.15f, 0.1f, 0.1f, 0.9f, 0.1f, 0.75f, 1.0f};
        jointGroupInputIndices.assign(inputIndices, inputIndices + 3ul);
        jointGroupOutputIndices.assign(outputIndices, outputIndices + 4ul);
        jointGroupValues.assign(values, values + 12ul);

        float vxs[] = {4.0f, 12.0f, 23.5f, -4.0f, 2.0f};
        float vys[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f};
        float vzs[] = {11.0f, -5.5f, 22.0f, 3.0f, 6.1f};
        vertexPositions.xs.assign(vxs, vxs + 5ul);
        vertexPositions.ys.assign(vys, vys + 5ul);
        vertexPositions.zs.assign(vzs, vzs + 5ul);

        float bxs[] = {4.0f, 12.0f, 23.5f, -4.0f, 2.0f};
        float bys[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f};
        float bzs[] = {11.0f, -5.5f, 22.0f, 3.0f, 6.1f};
        blendShapeTargetDeltas.xs.assign(bxs, bxs + 5ul);
        blendShapeTargetDeltas.ys.assign(bys, bys + 5ul);
        blendShapeTargetDeltas.zs.assign(bzs, bzs + 5ul);
    }

    std::uint16_t getJointParentIndex(std::uint16_t index) const override {
        return jointHierarchy[index];
    }

    std::uint16_t getJointCount() const override {
        return static_cast<std::uint16_t>(jointHierarchy.size());
    }

    dnac::StringView getJointName(std::uint16_t /*unused*/) const override {
        return dnac::StringView{"A", 1ul};
    }

    std::uint16_t getMeshCount() const override {
        return 1;
    }

    dnac::StringView getMeshName(std::uint16_t /*unused*/) const override {
        return dnac::StringView{"M", 1ul};
    }

    dnac::Vector3 getNeutralJointTranslation(std::uint16_t index) const override {
        return dnac::Vector3{neutralJointTranslations.xs[index],
                             neutralJointTranslations.ys[index],
                             neutralJointTranslations.zs[index]};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationXs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.xs};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationYs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.ys};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationZs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.zs};
    }

    std::uint16_t getJointGroupCount() const override {
        return jointGroupCount;
    }

    dnac::ConstArrayView<std::uint16_t> getJointGroupInputIndices(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<std::uint16_t>{jointGroupInputIndices};
    }

    dnac::ConstArrayView<std::uint16_t> getJointGroupOutputIndices(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<std::uint16_t>{jointGroupOutputIndices};
    }

    dnac::ConstArrayView<float> getJointGroupValues(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{jointGroupValues};
    }

    std::uint32_t getVertexPositionCount(std::uint16_t /*unused*/) const override {
        return static_cast<std::uint32_t>(vertexPositions.size());
    }

    dnac::Vector3 getVertexPosition(std::uint16_t /*unused*/, std::uint32_t vertexIndex) const override {
        return dnac::Vector3{vertexPositions.xs[vertexIndex], vertexPositions.ys[vertexIndex], vertexPositions.zs[vertexIndex]};
    }

    dnac::ConstArrayView<float> getVertexPositionXs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.xs};
    }

    dnac::ConstArrayView<float> getVertexPositionYs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.ys};
    }

    dnac::ConstArrayView<float> getVertexPositionZs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.zs};
    }

    std::uint16_t getBlendShapeTargetCount(std::uint16_t /*unused*/) const override {
        return 1u;
    }

    std::uint32_t getBlendShapeTargetDeltaCount(std::uint16_t /*unused*/, std::uint16_t /*unused*/) const override {
        return static_cast<std::uint32_t>(blendShapeTargetDeltas.size());
    }

    dnac::Vector3 getBlendShapeTargetDelta(std::uint16_t /*unused*/,
                                           std::uint16_t /*unused*/,
                                           std::uint32_t deltaIndex) const override {
        return dnac::Vector3{blendShapeTargetDeltas.xs[deltaIndex],
                             blendShapeTargetDeltas.ys[deltaIndex],
                             blendShapeTargetDeltas.zs[deltaIndex]};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t /*unused*/, std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{blendShapeTargetDeltas.xs};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t /*unused*/, std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{blendShapeTargetDeltas.ys};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t /*unused*/, std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{blendShapeTargetDeltas.zs};
    }

private:
    std::uint16_t jointGroupCount;
    std::array<std::uint16_t, 2> jointHierarchy;
    dnac::RawVector3Vector neutralJointTranslations;

    dnac::Vector<std::uint16_t> jointGroupInputIndices;
    dnac::Vector<std::uint16_t> jointGroupOutputIndices;
    dnac::Vector<float> jointGroupValues;

    dnac::RawVector3Vector vertexPositions;
    dnac::RawVector3Vector blendShapeTargetDeltas;
};

class ScaleCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        ScalableDNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
        scale = 2.0f;
        origin = {0.0f, 3.0f, 0.0f};

        float jxs[] = {2.0f, 5.0f};
        float jys[] = {3.0f, 9.0f};
        float jzs[] = {8.0f, 16.0f};
        expectedNeutralJointTranslationXs.assign(jxs, jxs + 2ul);
        expectedNeutralJointTranslationYs.assign(jys, jys + 2ul);
        expectedNeutralJointTranslationZs.assign(jzs, jzs + 2ul);

        float values[] = {1.0f, 0.4f, 0.6f, 0.5f, 0.8f, 0.3f, 0.1f, 0.1f, 0.9f, 0.2f, 1.5f, 2.0f};
        expectedJointGroupValues.assign(values, values + 12ul);

        float vxs[] = {8.0f, 24.0f, 47.0f, -8.0f, 4.0f};
        float vys[] = {-1.0f, 1.0f, 3.0f, 5.0f, 7.0f};
        float vzs[] = {22.0f, -11.0f, 44.0f, 6.0f, 12.2f};
        expectedVertexPositionXs.assign(vxs, vxs + 5ul);
        expectedVertexPositionYs.assign(vys, vys + 5ul);
        expectedVertexPositionZs.assign(vzs, vzs + 5ul);

        float bxs[] = {8.0f, 24.0f, 47.0f, -8.0f, 4.0f};
        float bys[] = {2.0f, 4.0f, 6.0f, 8.0f, 10.0f};
        float bzs[] = {22.0f, -11.0f, 44.0f, 6.0f, 12.2f};
        expectedBlendShapeTargetDeltaXs.assign(bxs, bxs + 5ul);
        expectedBlendShapeTargetDeltaYs.assign(bys, bys + 5ul);
        expectedBlendShapeTargetDeltaZs.assign(bzs, bzs + 5ul);
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;

    float scale;
    dnac::Vector3 origin;

    dnac::Vector<float> expectedNeutralJointTranslationXs;
    dnac::Vector<float> expectedNeutralJointTranslationYs;
    dnac::Vector<float> expectedNeutralJointTranslationZs;
    dnac::Vector<float> expectedJointGroupValues;
    dnac::Vector<float> expectedVertexPositionXs;
    dnac::Vector<float> expectedVertexPositionYs;
    dnac::Vector<float> expectedVertexPositionZs;
    dnac::Vector<float> expectedBlendShapeTargetDeltaXs;
    dnac::Vector<float> expectedBlendShapeTargetDeltaYs;
    dnac::Vector<float> expectedBlendShapeTargetDeltaZs;
};

}  // namespace

TEST_F(ScaleCommandTest, DoubleUpWithNonZeroOrigin) {
    dnac::ScaleCommand setCmd(scale, origin);
    setCmd.run(output.get());

    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationXs(),
                         expectedNeutralJointTranslationXs,
                         expectedNeutralJointTranslationXs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationYs(),
                         expectedNeutralJointTranslationYs,
                         expectedNeutralJointTranslationYs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationZs(),
                         expectedNeutralJointTranslationZs,
                         expectedNeutralJointTranslationZs.size(),
                         0.0001);

    ASSERT_ELEMENTS_NEAR(output->getJointGroupValues(0), expectedJointGroupValues, expectedJointGroupValues.size(), 0.0001);

    ASSERT_ELEMENTS_NEAR(output->getVertexPositionXs(0), expectedVertexPositionXs, expectedVertexPositionXs.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionYs(0), expectedVertexPositionYs, expectedVertexPositionYs.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionZs(0), expectedVertexPositionZs, expectedVertexPositionZs.size(), 0.0001);

    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(0, 0),
                         expectedBlendShapeTargetDeltaXs,
                         expectedBlendShapeTargetDeltaXs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(0, 0),
                         expectedBlendShapeTargetDeltaYs,
                         expectedBlendShapeTargetDeltaYs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(0, 0),
                         expectedBlendShapeTargetDeltaZs,
                         expectedBlendShapeTargetDeltaZs.size(),
                         0.0001);
}
