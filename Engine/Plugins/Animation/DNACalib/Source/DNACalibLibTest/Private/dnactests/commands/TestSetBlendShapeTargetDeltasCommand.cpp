// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/SetBlendShapeTargetDeltasCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"
#include "dnactests/Defs.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"
#include "dnactests/commands/BlendShapeDNAReader.h"
#include "pma/ScopedPtr.h"
#include <cstdint>

namespace {

class SetBlendShapeTargetDeltasDNAReader : public dna::FakeDNACReader {
public:
    explicit SetBlendShapeTargetDeltasDNAReader(dnac::MemoryResource* memRes = nullptr) :
        blendShapeNames{memRes},
        meshNames{memRes},
        bsChannelIndices{memRes},
        bsTargetDeltas{memRes},
        bsTargetVertexIndices{memRes},
        vertexCounts{memRes} {

        blendShapeNames.assign(
            {{"blendshape1", memRes}, {"blendshape2", memRes}, {"blendshape3", memRes}, {"blendshape4", memRes}});
        meshNames.assign({{"mesh1", memRes}, {"mesh2", memRes}});

        bsChannelIndices.assign({{0u, 1u, 2u}, {3u}});

        const auto meshCount = SetBlendShapeTargetDeltasDNAReader::getMeshCount();
        bsTargetDeltas.resize(meshCount);
        bsTargetDeltas[0].resize(3u, dnac::RawVector3Vector{memRes});
        bsTargetDeltas[1].resize(1u, dnac::RawVector3Vector{memRes});

        bsTargetVertexIndices.resize(meshCount);
        bsTargetVertexIndices[0].resize(3u);
        bsTargetVertexIndices[1].resize(1u);

        vertexCounts = {10u, 6u};
    }

    std::uint16_t getMeshCount() const override {
        return static_cast<std::uint16_t>(meshNames.size());
    }

    dnac::StringView getMeshName(std::uint16_t index) const override {
        return dnac::StringView{meshNames[index]};
    }

    std::uint16_t getBlendShapeChannelCount() const override {
        return static_cast<std::uint16_t>(blendShapeNames.size());
    }

    dnac::StringView getBlendShapeChannelName(std::uint16_t index) const override {
        return dnac::StringView{blendShapeNames[index]};
    }

    std::uint16_t getBlendShapeTargetCount(std::uint16_t meshIndex) const override {
        if (meshIndex < getMeshCount()) {
            return static_cast<std::uint16_t>(bsChannelIndices[meshIndex].size());
        }
        return {};
    }

    std::uint16_t getBlendShapeChannelIndex(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return bsChannelIndices[meshIndex][blendShapeTargetIndex];
            }
        }
        return {};
    }

    std::uint32_t getBlendShapeTargetDeltaCount(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return static_cast<std::uint32_t>(bsTargetDeltas[meshIndex][blendShapeTargetIndex].size());
            }
        }
        return {};
    }

    dnac::Vector3 getBlendShapeTargetDelta(std::uint16_t meshIndex,
                                           std::uint16_t blendShapeTargetIndex,
                                           std::uint32_t deltaIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                if (deltaIndex < getBlendShapeTargetDeltaCount(meshIndex, blendShapeTargetIndex)) {
                    return dnac::Vector3{bsTargetDeltas[meshIndex][blendShapeTargetIndex].xs[deltaIndex],
                                         bsTargetDeltas[meshIndex][blendShapeTargetIndex].ys[deltaIndex],
                                         bsTargetDeltas[meshIndex][blendShapeTargetIndex].zs[deltaIndex]};
                }
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].xs};
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].ys};
            }
        }
        return {};
    }

    dnac::ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t meshIndex,
                                                           std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<float>{bsTargetDeltas[meshIndex][blendShapeTargetIndex].zs};
            }
        }
        return {};
    }

    dnac::ConstArrayView<std::uint32_t> getBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                                         std::uint16_t blendShapeTargetIndex) const override {
        if (meshIndex < getMeshCount()) {
            if (blendShapeTargetIndex < getBlendShapeTargetCount(meshIndex)) {
                return dnac::ConstArrayView<std::uint32_t>{bsTargetVertexIndices[meshIndex][blendShapeTargetIndex]};
            }
        }
        return {};
    }

    std::uint32_t getVertexPositionCount(std::uint16_t meshIndex) const override {
        if (meshIndex < getMeshCount()) {
            return vertexCounts[meshIndex];
        }
        return {};
    }

private:
    dnac::Vector<dnac::String<char>> blendShapeNames;
    dnac::Vector<dnac::String<char>> meshNames;

    dnac::Matrix<std::uint16_t> bsChannelIndices;

    dnac::Matrix<dnac::RawVector3Vector> bsTargetDeltas;
    dnac::Matrix<dnac::Vector<std::uint32_t>> bsTargetVertexIndices;

    dnac::Vector<std::uint32_t> vertexCounts;
};

class SetBlendShapeTargetDeltasCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        sc::StatusProvider provider{};
        provider.reset();
        ASSERT_TRUE(dnac::Status::isOk());
        meshIndex = 0u;
        blendShapeTargetIndex = 0u;
        SetBlendShapeTargetDeltasDNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
        deltas = {{0.0f, 0.0f, 0.0f}, {1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}};
        vertexIndices = {0u, 1u, 2u};
    }

    void TearDown() override {
    }

protected:
    std::uint16_t meshIndex;
    std::uint16_t blendShapeTargetIndex;
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;
    dnac::Vector<dnac::Vector3> deltas;
    dnac::Vector<std::uint32_t> vertexIndices;
};

}  // namespace

TEST_F(SetBlendShapeTargetDeltasCommandTest, InterpolateDeltas) {
    // Set deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand setCmd(meshIndex,
                                                  blendShapeTargetIndex,
                                                  dnac::ConstArrayView<dnac::Vector3>{deltas},
                                                  dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                                  dnac::VectorOperation::Interpolate);
    setCmd.run(output.get());

    const dnac::Vector<float> expectedOnEmpty = {1.0f, 2.0f};
    const dnac::Vector<std::uint32_t> expectedVtxIndicesOnEmpty = {1u, 2u};

    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());

    dnac::Vector<dnac::Vector3> deltasOther = {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}};
    dnac::Vector<float> masks = {0.5f, 0.5f, 0.5f};
    // Interpolate deltas on non-empty output
    dnac::SetBlendShapeTargetDeltasCommand interpolateCmd(meshIndex,
                                                          blendShapeTargetIndex,
                                                          dnac::ConstArrayView<dnac::Vector3>{deltasOther},
                                                          dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                                          dnac::ConstArrayView<float>{masks},
                                                          dnac::VectorOperation::Interpolate);
    interpolateCmd.run(output.get());

    const dnac::Vector<float> expectedOnNonEmpty = {0.5f, 1.5f, 2.5f};
    const dnac::Vector<std::uint32_t> expectedVtxIndicesOnNonEmpty = {0u, 1u, 2u};

    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnNonEmpty.size());
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                         expectedOnNonEmpty,
                         expectedOnNonEmpty.size(),
                         0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                         expectedOnNonEmpty,
                         expectedOnNonEmpty.size(),
                         0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                         expectedOnNonEmpty,
                         expectedOnNonEmpty.size(),
                         0.0001f);
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnNonEmpty,
                       expectedVtxIndicesOnNonEmpty.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, AddDeltas) {
    // Add deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand cmd(meshIndex,
                                               blendShapeTargetIndex,
                                               dnac::ConstArrayView<dnac::Vector3>{deltas},
                                               dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                               dnac::VectorOperation::Add);
    cmd.run(output.get());

    const dnac::Vector<float> expectedOnEmpty = {1.0f, 2.0f};
    const dnac::Vector<std::uint32_t> expectedVtxIndicesOnEmpty = {1u, 2u};

    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());

    // Add deltas on non-empty output
    cmd.run(output.get());

    const dnac::Vector<float> expectedOnNonEmpty = {2.0f, 4.0f};

    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, SubtractDeltas) {
    // Subtract deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand cmd(meshIndex,
                                               blendShapeTargetIndex,
                                               dnac::ConstArrayView<dnac::Vector3>{deltas},
                                               dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                               dnac::VectorOperation::Subtract);
    cmd.run(output.get());

    const dnac::Vector<float> expectedOnEmpty = {-1.0f, -2.0f};
    const dnac::Vector<std::uint32_t> expectedVtxIndicesOnEmpty = {1u, 2u};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());

    // Subtract deltas on non-empty output
    cmd.run(output.get());

    const dnac::Vector<float> expectedOnNonEmpty = {-2.0f, -4.0f};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, MultiplyDeltas) {
    // Set deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand cmd(meshIndex,
                                               blendShapeTargetIndex,
                                               dnac::ConstArrayView<dnac::Vector3>{deltas},
                                               dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                               dnac::VectorOperation::Interpolate);
    cmd.run(output.get());

    const dnac::Vector<float> expectedOnEmpty = {1.0f, 2.0f};
    const dnac::Vector<std::uint32_t> expectedVtxIndicesOnEmpty = {1u, 2u};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());

    dnac::Vector<dnac::Vector3> deltasOther = {{2.0f, 2.0f, 2.0f}, {4.0f, 4.0f, 4.0f}, {6.0f, 6.0f, 6.0f}};
    // Multiply deltas on non-empty output
    dnac::SetBlendShapeTargetDeltasCommand mulCmd(meshIndex,
                                                  blendShapeTargetIndex,
                                                  dnac::ConstArrayView<dnac::Vector3>{deltasOther},
                                                  dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                                  dnac::VectorOperation::Multiply);
    mulCmd.run(output.get());
    const dnac::Vector<float> expectedOnNonEmpty = {4.0f, 12.0f};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, OverwriteDeltas) {
    // Set deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand cmd(meshIndex,
                                               blendShapeTargetIndex,
                                               dnac::ConstArrayView<dnac::Vector3>{deltas},
                                               dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                               dnac::VectorOperation::Interpolate);
    cmd.run(output.get());

    const dnac::Vector<float> expected = {1.0f, 2.0f};
    const dnac::Vector<std::uint32_t> expectedVtxIndices = {1u, 2u};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expected.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expected.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expected.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(), expectedVtxIndices.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex), expected, expected.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex), expected, expected.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex), expected, expected.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndices,
                       expectedVtxIndices.size());

    dnac::Vector<dnac::Vector3> deltasOther = {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}};
    // Overwrite deltas on non-empty output
    dnac::SetBlendShapeTargetDeltasCommand overwriteCmd(meshIndex,
                                                        blendShapeTargetIndex,
                                                        dnac::ConstArrayView<dnac::Vector3>{deltasOther},
                                                        dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                                        dnac::VectorOperation::Interpolate);
    overwriteCmd.run(output.get());

    const dnac::Vector<float> expectedOnNonEmpty = {1.0f, 2.0f, 3.0f};
    const dnac::Vector<std::uint32_t> expectedVtxIndicesOnNonEmpty = {0u, 1u, 2u};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnNonEmpty,
                       expectedVtxIndicesOnNonEmpty.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, SetFewerDeltas) {
    // Set deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand setCmd(meshIndex,
                                                  blendShapeTargetIndex,
                                                  dnac::ConstArrayView<dnac::Vector3>{deltas},
                                                  dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                                  dnac::VectorOperation::Interpolate);
    setCmd.run(output.get());

    const dnac::Vector<float> expectedOnEmpty = {1.0f, 2.0f};
    const dnac::Vector<std::uint32_t> expectedVtxIndicesOnEmpty = {1u, 2u};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());

    dnac::Vector<dnac::Vector3> deltasOther = {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}};
    dnac::Vector<std::uint32_t> vertexIndicesOther = {0u, 2u};
    dnac::Vector<float> masks = {1.0f, 1.0f};

    // Set fewer deltas
    dnac::SetBlendShapeTargetDeltasCommand interpolateCmd(meshIndex,
                                                          blendShapeTargetIndex,
                                                          dnac::ConstArrayView<dnac::Vector3>{deltasOther},
                                                          dnac::ConstArrayView<std::uint32_t>{vertexIndicesOther},
                                                          dnac::ConstArrayView<float>{masks},
                                                          dnac::VectorOperation::Interpolate);
    interpolateCmd.run(output.get());

    const dnac::Vector<float> expectedOnNonEmpty = {1.0f, 1.0f, 2.0f};
    const dnac::Vector<float> expectedVtxIndicesOnNonEmpty = {0u, 1u, 2u};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnNonEmpty.size());
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                         expectedOnNonEmpty,
                         expectedOnNonEmpty.size(),
                         0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                         expectedOnNonEmpty,
                         expectedOnNonEmpty.size(),
                         0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                         expectedOnNonEmpty,
                         expectedOnNonEmpty.size(),
                         0.0001f);
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnNonEmpty,
                       expectedVtxIndicesOnNonEmpty.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, NonAscendingVertexIndices) {
    // Add deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand cmd(meshIndex,
                                               blendShapeTargetIndex,
                                               dnac::ConstArrayView<dnac::Vector3>{deltas},
                                               dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                               dnac::VectorOperation::Add);
    cmd.run(output.get());
    const dnac::Vector<float> expectedOnEmpty = {1.0f, 2.0f};
    const dnac::Vector<std::uint32_t> expectedVtxIndicesOnEmpty = {1u, 2u};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());

    vertexIndices = {7u, 2u, 0u, 1u};
    deltas = {{3.0f, 3.0f, 3.0f}, {0.0f, 0.0f, 0.0f}, {1.0f, 1.0f, 1.0f}, {-1.0f, -1.0f, -1.0f}};
    cmd.setVertexIndices(dnac::ConstArrayView<std::uint32_t>{vertexIndices});
    cmd.setDeltas(dnac::ConstArrayView<dnac::Vector3>{deltas});
    // Add deltas on non-empty output
    cmd.run(output.get());
    const dnac::Vector<float> expectedOnNonEmpty = {1.0f, 2.0f, 3.0f};
    const dnac::Vector<std::uint32_t> expectedVtxIndicesOnNonEmpty = {0u, 2u, 7u};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnNonEmpty,
                       expectedVtxIndicesOnNonEmpty.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, EmptyVertexIndices) {
    // Set deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand cmd(meshIndex,
                                               blendShapeTargetIndex,
                                               dnac::ConstArrayView<dnac::Vector3>{deltas},
                                               dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                               dnac::VectorOperation::Interpolate);
    cmd.run(output.get());

    const dnac::Vector<float> expectedOnEmpty = {1.0f, 2.0f};
    const dnac::Vector<std::uint32_t> expectedVtxIndicesOnEmpty = {1u, 2u};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());

    dnac::Vector<dnac::Vector3> deltasOther = {{4.0f, 4.0f, 4.0f}, {6.0f, 6.0f, 6.0f}};
    // Multiply deltas on non-empty output
    dnac::SetBlendShapeTargetDeltasCommand mulCmd(meshIndex,
                                                  blendShapeTargetIndex,
                                                  dnac::ConstArrayView<dnac::Vector3>{deltasOther},
                                                  dnac::ConstArrayView<std::uint32_t>{},
                                                  dnac::VectorOperation::Multiply);
    mulCmd.run(output.get());
    const dnac::Vector<float> expectedOnNonEmpty = {4.0f, 12.0f};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnNonEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnNonEmpty,
                       expectedOnNonEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, SetDeltasForAllVertices) {
    deltas = {{1.0f, 1.0f, 1.0f},
              {2.0f, 2.0f, 2.0f},
              {3.0f, 3.0f, 3.0f},
              {4.0f, 4.0f, 4.0f},
              {5.0f, 5.0f, 5.0f},
              {6.0f, 6.0f, 6.0f},
              {7.0f, 7.0f, 7.0f},
              {8.0f, 8.0f, 8.0f},
              {9.0f, 9.0f, 9.0f},
              {10.0f, 10.0f, 10.0f}};
    vertexIndices = {0u, 1u, 2u, 3u, 4u, 5u, 6u, 7u, 8u, 9u};
    // Set deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand cmd(meshIndex,
                                               blendShapeTargetIndex,
                                               dnac::ConstArrayView<dnac::Vector3>{deltas},
                                               dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                               dnac::VectorOperation::Interpolate);
    cmd.run(output.get());

    const dnac::Vector<float> expectedOnEmpty = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f, 10.0f};
    const dnac::Vector<std::uint32_t> expectedVtxIndicesOnEmpty = {0u, 1u, 2u, 3u, 4u, 5u, 6u, 7u, 8u, 9u};
    ASSERT_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex).size(), expectedOnEmpty.size());
    ASSERT_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex).size(),
              expectedVtxIndicesOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex),
                       expectedOnEmpty,
                       expectedOnEmpty.size());
    ASSERT_ELEMENTS_EQ(output->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex),
                       expectedVtxIndicesOnEmpty,
                       expectedVtxIndicesOnEmpty.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, VertexIndexOutOfBounds) {
    // Add deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand cmd(meshIndex,
                                               blendShapeTargetIndex,
                                               dnac::ConstArrayView<dnac::Vector3>{deltas},
                                               dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                               dnac::VectorOperation::Add);
    vertexIndices = {0u, 1u, 10u};
    deltas = {{3.0f, 3.0f, 3.0f}, {1.0f, 1.0f, 1.0f}, {-1.0f, -1.0f, -1.0f}};
    cmd.setVertexIndices(dnac::ConstArrayView<std::uint32_t>{vertexIndices});
    cmd.setDeltas(dnac::ConstArrayView<dnac::Vector3>{deltas});
    cmd.run(output.get());
    const auto error = dnac::Status::get();
    ASSERT_EQ(error, dnac::SetBlendShapeTargetDeltasCommand::VertexIndicesOutOfBoundsError);
    ASSERT_STREQ(error.message, "Vertex index (10) is out of bounds. Vertex count is (10).");
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, NoVertexIndicesSet) {
    // Add deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand cmd;
    cmd.setMeshIndex(meshIndex);
    cmd.setBlendShapeTargetIndex(blendShapeTargetIndex);
    cmd.setDeltas(dnac::ConstArrayView<dnac::Vector3>{deltas});
    cmd.setOperation(dnac::VectorOperation::Add);
    cmd.run(output.get());
    const auto error = dnac::Status::get();
    ASSERT_EQ(error, dnac::SetBlendShapeTargetDeltasCommand::NoVertexIndicesSetError);
    ASSERT_STREQ(error.message,
                 "No vertex indices set. Current vertex indices in DNA will not be used, as their number (0) differs from the "
                 "number of set deltas (3).");
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, DeltasVertexIndicesCountMismatch) {
    // Add deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand cmd(meshIndex,
                                               blendShapeTargetIndex,
                                               dnac::ConstArrayView<dnac::Vector3>{deltas},
                                               dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                               dnac::VectorOperation::Add);
    vertexIndices = {0u, 1u};
    deltas = {{3.0f, 3.0f, 3.0f}, {1.0f, 1.0f, 1.0f}, {-1.0f, -1.0f, -1.0f}};
    cmd.setVertexIndices(dnac::ConstArrayView<std::uint32_t>{vertexIndices});
    cmd.setDeltas(dnac::ConstArrayView<dnac::Vector3>{deltas});
    cmd.run(output.get());
    const auto error = dnac::Status::get();
    ASSERT_EQ(error, dnac::SetBlendShapeTargetDeltasCommand::DeltasVertexIndicesCountMismatch);
    ASSERT_STREQ(error.message, "Number of set deltas (3) differs from number of set vertex indices (2).");
}

TEST_F(SetBlendShapeTargetDeltasCommandTest, DeltasMasksCountMismatch) {
    // Add deltas on empty output
    dnac::SetBlendShapeTargetDeltasCommand cmd(meshIndex,
                                               blendShapeTargetIndex,
                                               dnac::ConstArrayView<dnac::Vector3>{deltas},
                                               dnac::ConstArrayView<std::uint32_t>{vertexIndices},
                                               dnac::VectorOperation::Add);
    vertexIndices = {0u, 1u, 2u};
    deltas = {{3.0f, 3.0f, 3.0f}, {1.0f, 1.0f, 1.0f}, {-1.0f, -1.0f, -1.0f}};
    dnac::Vector<float> masks = {0.5f, 0.5f, 0.5f, 0.7f};
    cmd.setVertexIndices(dnac::ConstArrayView<std::uint32_t>{vertexIndices});
    cmd.setDeltas(dnac::ConstArrayView<dnac::Vector3>{deltas});
    cmd.setMasks(dnac::ConstArrayView<float>{masks});
    cmd.run(output.get());
    const auto error = dnac::Status::get();
    ASSERT_EQ(error, dnac::SetBlendShapeTargetDeltasCommand::DeltasMasksCountMismatch);
    ASSERT_STREQ(error.message, "Number of set deltas (3) differs from number of set masks (4).");
}
