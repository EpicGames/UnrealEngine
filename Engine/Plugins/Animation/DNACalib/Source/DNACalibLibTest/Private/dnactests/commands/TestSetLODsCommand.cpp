// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"

#include <array>
#include <cstdint>

namespace {

class LODsDNAReader : public dna::FakeDNACReader {
public:
    LODsDNAReader(std::uint16_t lodCount_) :
        lodCount{lodCount_} {
    }

    std::uint16_t getLODCount() const override {
        return lodCount;
    }

private:
    std::uint16_t lodCount;
};

class SetLODsCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        LODsDNAReader fixtures{8u};
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;
};

}  // namespace

TEST_F(SetLODsCommandTest, SetLOD0) {
    std::uint16_t lod = 0u;
    dnac::SetLODsCommand cmd(dnac::ConstArrayView<std::uint16_t>{&lod, 1u});
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 1u);
}

TEST_F(SetLODsCommandTest, SetLOD1) {
    std::uint16_t lod = 1u;
    dnac::SetLODsCommand cmd(dnac::ConstArrayView<std::uint16_t>{&lod, 1u});
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 1u);
}

TEST_F(SetLODsCommandTest, SetLOD4) {
    std::uint16_t lod = 4u;
    dnac::SetLODsCommand cmd(dnac::ConstArrayView<std::uint16_t>{&lod, 1u});
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 1u);
}

TEST_F(SetLODsCommandTest, SetLOD13) {
    std::array<std::uint16_t, 2> lods = {1u, 3u};
    dnac::SetLODsCommand cmd(dnac::ConstArrayView<std::uint16_t>{lods});
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 2u);
}

TEST_F(SetLODsCommandTest, SetLOD03) {
    std::array<std::uint16_t, 2> lods = {0u, 3u};
    dnac::SetLODsCommand cmd(dnac::ConstArrayView<std::uint16_t>{lods});
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 2u);
}

TEST_F(SetLODsCommandTest, SetLOD26) {
    std::array<std::uint16_t, 2> lods = {2u, 6u};
    dnac::SetLODsCommand cmd(dnac::ConstArrayView<std::uint16_t>{lods});
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 2u);
}

TEST_F(SetLODsCommandTest, SetLOD246) {
    std::array<std::uint16_t, 3> lods = {2u, 4u, 6u};
    dnac::SetLODsCommand cmd(dnac::ConstArrayView<std::uint16_t>{lods});
    cmd.run(output.get());

    ASSERT_EQ(output->getLODCount(), 3u);
}
