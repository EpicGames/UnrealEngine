// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"

#include <cstdint>

namespace {

class SkinWeightsDNAReader : public dna::FakeDNACReader {
public:
    explicit SkinWeightsDNAReader(dnac::MemoryResource* memRes = nullptr) :
        skinWeightsValues{memRes},
        skinWeightsJointIndices{memRes} {

        skinWeightsJointIndices.resize(4u);
        skinWeightsJointIndices[0].assign({0, 1, 2});
        skinWeightsJointIndices[1].assign({0, 1});
        skinWeightsJointIndices[2].assign({1, 2});
        skinWeightsJointIndices[3].assign({1});

        skinWeightsValues.resize(4u);
        skinWeightsValues[0].assign({0.1f, 0.7f, 0.2f});
        skinWeightsValues[1].assign({0.2f, 0.8f});
        skinWeightsValues[2].assign({0.4f, 0.6f});
        skinWeightsValues[3].assign({1.0f});
    }

    std::uint16_t getLODCount() const override {
        return 1;
    }

    std::uint16_t getMeshCount() const override {
        return 1;
    }

    dnac::StringView getMeshName(std::uint16_t /*unused*/) const override {
        return dnac::StringView{"M", 1ul};
    }

    std::uint32_t getSkinWeightsCount(std::uint16_t /*unused*/) const override {
        return static_cast<std::uint32_t>(skinWeightsJointIndices.size());
    }

    dnac::ConstArrayView<float> getSkinWeightsValues(std::uint16_t /*unused*/, std::uint32_t vertexIndex) const override {
        return dnac::ConstArrayView<float>{skinWeightsValues[vertexIndex]};
    }

    dnac::ConstArrayView<std::uint16_t> getSkinWeightsJointIndices(std::uint16_t /*unused*/,
                                                                   std::uint32_t vertexIndex) const override {
        return dnac::ConstArrayView<std::uint16_t>{skinWeightsJointIndices[vertexIndex]};
    }

private:
    dnac::Matrix<float> skinWeightsValues;
    dnac::Matrix<std::uint16_t> skinWeightsJointIndices;
};

class SetSkinWeightsCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        SkinWeightsDNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;
};

}  // namespace

TEST_F(SetSkinWeightsCommandTest, UpdateSkinWeights) {
    static const float weights[] = {0.5f, 0.5f};
    static const std::uint16_t jointIndices[] = {3, 4};
    dnac::SetSkinWeightsCommand cmd(0u,
                                    0u,
                                    dnac::ConstArrayView<float>{weights, 2ul},
                                    dnac::ConstArrayView<std::uint16_t>{jointIndices, 2ul});
    cmd.run(output.get());

    ASSERT_EQ(output->getSkinWeightsCount(0), 4u);

    static const std::uint16_t expectedSWJointIndices0[] = {3, 4};
    static const std::uint16_t expectedSWJointIndices1[] = {0, 1};
    static const std::uint16_t expectedSWJointIndices2[] = {1, 2};
    static const std::uint16_t expectedSWJointIndices3[] = {1};

    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 0),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices0, 2ul}),
                       2ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 1),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices1, 2ul}),
                       2ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 2),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices2, 2ul}),
                       2ul);
    ASSERT_ELEMENTS_EQ(output->getSkinWeightsJointIndices(0, 3),
                       (dnac::ConstArrayView<std::uint16_t>{expectedSWJointIndices3, 1ul}),
                       1ul);

    static const float expectedSWValues0[] = {0.5f, 0.5f};
    static const float expectedSWValues1[] = {0.2f, 0.8f};
    static const float expectedSWValues2[] = {0.4f, 0.6f};
    static const float expectedSWValues3[] = {1.0f};

    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 0), (dnac::ConstArrayView<float>{expectedSWValues0, 2ul}), 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 1), (dnac::ConstArrayView<float>{expectedSWValues1, 2ul}), 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 2), (dnac::ConstArrayView<float>{expectedSWValues2, 2ul}), 2ul, 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getSkinWeightsValues(0, 3), (dnac::ConstArrayView<float>{expectedSWValues3, 1ul}), 1ul, 0.0001f);
}
