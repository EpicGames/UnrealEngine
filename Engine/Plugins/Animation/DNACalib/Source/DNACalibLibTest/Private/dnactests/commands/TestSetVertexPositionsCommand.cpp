// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnacalib/commands/SetVertexPositionsCommand.h"
#include "dnactests/Defs.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"

namespace {

class SetVertexPositionsCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        sc::StatusProvider provider{};
        provider.reset();
        ASSERT_TRUE(dnac::Status::isOk());
        meshIndex = 0u;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>();
        positions = {{0.0f, 0.0f, 0.0f}, {1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}};
    }

    void TearDown() override {
    }

protected:
    std::uint16_t meshIndex;
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;
    dnac::Vector<dnac::Vector3> positions;
};

}  // namespace

TEST_F(SetVertexPositionsCommandTest, InterpolatePositions) {
    // Set vertices on empty output
    dnac::SetVertexPositionsCommand setCmd(meshIndex,
                                           dnac::ConstArrayView<dnac::Vector3>{positions},
                                           dnac::VectorOperation::Interpolate);
    setCmd.run(output.get());

    const float expectedOnEmpty[] = {0.0f, 1.0f, 2.0f};
    ASSERT_ELEMENTS_EQ(output->getVertexPositionXs(meshIndex), expectedOnEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionYs(meshIndex), expectedOnEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionZs(meshIndex), expectedOnEmpty, positions.size());

    dnac::Vector<dnac::Vector3> positionsOther{{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}};
    dnac::Vector<float> masks{0.5f, 0.5f, 0.5f};
    // Interpolate vertices on non-empty output
    dnac::SetVertexPositionsCommand interpolateCmd(meshIndex,
                                                   dnac::ConstArrayView<dnac::Vector3>{positionsOther},
                                                   dnac::ConstArrayView<float>{masks},
                                                   dnac::VectorOperation::Interpolate);
    interpolateCmd.run(output.get());

    const dnac::Vector<float> expectedOnNonEmpty{0.5f, 1.5f, 2.5f};
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionXs(meshIndex), expectedOnNonEmpty, positions.size(), 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionYs(meshIndex), expectedOnNonEmpty, positions.size(), 0.0001f);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionZs(meshIndex), expectedOnNonEmpty, positions.size(), 0.0001f);
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetVertexPositionsCommandTest, AddPositions) {
    // Add vertices on empty output
    dnac::SetVertexPositionsCommand cmd(meshIndex, dnac::ConstArrayView<dnac::Vector3>{positions}, dnac::VectorOperation::Add);
    cmd.run(output.get());

    const float expectedOnEmpty[] = {0.0f, 1.0f, 2.0f};
    ASSERT_ELEMENTS_EQ(output->getVertexPositionXs(meshIndex), expectedOnEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionYs(meshIndex), expectedOnEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionZs(meshIndex), expectedOnEmpty, positions.size());

    // Add vertices on non-empty output
    cmd.run(output.get());

    const float expectedOnNonEmpty[] = {0.0f, 2.0f, 4.0f};
    ASSERT_ELEMENTS_EQ(output->getVertexPositionXs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionYs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionZs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetVertexPositionsCommandTest, SubtractPositions) {
    // Subtract vertices on empty output
    dnac::SetVertexPositionsCommand cmd(meshIndex,
                                        dnac::ConstArrayView<dnac::Vector3>{positions},
                                        dnac::VectorOperation::Subtract);
    cmd.run(output.get());

    const float expectedOnEmpty[] = {0.0f, -1.0f, -2.0f};
    ASSERT_ELEMENTS_EQ(output->getVertexPositionXs(meshIndex), expectedOnEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionYs(meshIndex), expectedOnEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionZs(meshIndex), expectedOnEmpty, positions.size());

    // Subtract vertices on non-empty output
    cmd.run(output.get());

    const float expectedOnNonEmpty[] = {0.0f, -2.0f, -4.0f};
    ASSERT_ELEMENTS_EQ(output->getVertexPositionXs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionYs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionZs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetVertexPositionsCommandTest, MultiplyPositions) {
    // Set vertices on empty output
    dnac::SetVertexPositionsCommand cmd(meshIndex,
                                        dnac::ConstArrayView<dnac::Vector3>{positions},
                                        dnac::VectorOperation::Interpolate);
    cmd.run(output.get());

    const float expectedOnEmpty[] = {0.0f, 1.0f, 2.0f};
    ASSERT_ELEMENTS_EQ(output->getVertexPositionXs(meshIndex), expectedOnEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionYs(meshIndex), expectedOnEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionZs(meshIndex), expectedOnEmpty, positions.size());

    dnac::Vector<dnac::Vector3> positionsOther{{2.0f, 2.0f, 2.0f}, {4.0f, 4.0f, 4.0f}, {6.0f, 6.0f, 6.0f}};
    // Multiply vertices on non-empty output
    dnac::SetVertexPositionsCommand mulCmd(meshIndex,
                                           dnac::ConstArrayView<dnac::Vector3>{positionsOther},
                                           dnac::VectorOperation::Multiply);
    mulCmd.run(output.get());
    const float expectedOnNonEmpty[] = {0.0f, 4.0f, 12.0f};
    ASSERT_ELEMENTS_EQ(output->getVertexPositionXs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionYs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionZs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetVertexPositionsCommandTest, OverwritePositions) {
    // Set vertices on empty output
    dnac::SetVertexPositionsCommand cmd(meshIndex,
                                        dnac::ConstArrayView<dnac::Vector3>{positions},
                                        dnac::VectorOperation::Interpolate);
    cmd.run(output.get());

    const float expected[] = {0.0f, 1.0f, 2.0f};
    ASSERT_ELEMENTS_EQ(output->getVertexPositionXs(meshIndex), expected, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionYs(meshIndex), expected, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionZs(meshIndex), expected, positions.size());

    dnac::Vector<dnac::Vector3> positionsOther{{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}};
    // Overwrite vertices on non-empty output
    dnac::SetVertexPositionsCommand overwriteCmd(meshIndex,
                                                 dnac::ConstArrayView<dnac::Vector3>{positionsOther},
                                                 dnac::VectorOperation::Interpolate);
    overwriteCmd.run(output.get());

    const dnac::Vector<float> expectedOnNonEmpty{1.0f, 2.0f, 3.0f};
    ASSERT_ELEMENTS_EQ(output->getVertexPositionXs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionYs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_ELEMENTS_EQ(output->getVertexPositionZs(meshIndex), expectedOnNonEmpty, positions.size());
    ASSERT_TRUE(dnac::Status::isOk());
}

TEST_F(SetVertexPositionsCommandTest, PositionsMasksCountMismatch) {
    // Set vertices on empty output
    dnac::Vector<float> masks{0.5f, 0.5f};
    // Interpolate vertices on non-empty output
    dnac::SetVertexPositionsCommand cmd(meshIndex,
                                        dnac::ConstArrayView<dnac::Vector3>{positions},
                                        dnac::ConstArrayView<float>{masks},
                                        dnac::VectorOperation::Interpolate);
    cmd.run(output.get());
    const auto error = dnac::Status::get();
    ASSERT_EQ(error, dnac::SetVertexPositionsCommand::PositionsMasksCountMismatch);
    ASSERT_STREQ(error.message, "Number of set positions (3) differs from number of set masks (2).");
}
