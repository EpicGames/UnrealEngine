// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"
#include "dnactests/commands/FakeDNACReader.h"

#include "dnacalib/DNACalib.h"
#include "dnacalib/TypeDefs.h"
#include "dnacalib/dna/DNA.h"

#include <array>
#include <cstdint>

namespace {

class TranslatableDNAReader : public dna::FakeDNACReader {
public:
    explicit TranslatableDNAReader(dnac::MemoryResource* memRes = nullptr) :
        neutralJointTranslations{memRes},
        vertexPositions{memRes} {

        jointHierarchy = {0, 0};
        float jxs[] = {1.0f, 2.5f};
        float jys[] = {3.0f, 4.5f};
        float jzs[] = {4.0f, 8.0f};
        neutralJointTranslations.xs.assign(jxs, jxs + 2ul);
        neutralJointTranslations.ys.assign(jys, jys + 2ul);
        neutralJointTranslations.zs.assign(jzs, jzs + 2ul);

        float vxs[] = {4.0f, 12.0f, 23.5f, -4.0f, 2.0f};
        float vys[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f};
        float vzs[] = {11.0f, -5.5f, 22.0f, 3.0f, 6.1f};
        vertexPositions.xs.assign(vxs, vxs + 5ul);
        vertexPositions.ys.assign(vys, vys + 5ul);
        vertexPositions.zs.assign(vzs, vzs + 5ul);
    }

    std::uint16_t getJointParentIndex(std::uint16_t index) const override {
        return jointHierarchy[index];
    }

    std::uint16_t getJointCount() const override {
        return static_cast<std::uint16_t>(jointHierarchy.size());
    }

    dnac::StringView getJointName(std::uint16_t /*unused*/) const override {
        return dnac::StringView{"A", 1ul};
    }

    std::uint16_t getMeshCount() const override {
        return 1;
    }

    dnac::StringView getMeshName(std::uint16_t /*unused*/) const override {
        return dnac::StringView{"M", 1ul};
    }

    dnac::Vector3 getNeutralJointTranslation(std::uint16_t index) const override {
        return dnac::Vector3{neutralJointTranslations.xs[index],
                             neutralJointTranslations.ys[index],
                             neutralJointTranslations.zs[index]};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationXs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.xs};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationYs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.ys};
    }

    dnac::ConstArrayView<float> getNeutralJointTranslationZs() const override {
        return dnac::ConstArrayView<float>{neutralJointTranslations.zs};
    }

    std::uint32_t getVertexPositionCount(std::uint16_t /*unused*/) const override {
        return static_cast<std::uint32_t>(vertexPositions.size());
    }

    dnac::Vector3 getVertexPosition(std::uint16_t /*unused*/, std::uint32_t vertexIndex) const override {
        return dnac::Vector3{vertexPositions.xs[vertexIndex], vertexPositions.ys[vertexIndex], vertexPositions.zs[vertexIndex]};
    }

    dnac::ConstArrayView<float> getVertexPositionXs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.xs};
    }

    dnac::ConstArrayView<float> getVertexPositionYs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.ys};
    }

    dnac::ConstArrayView<float> getVertexPositionZs(std::uint16_t /*unused*/) const override {
        return dnac::ConstArrayView<float>{vertexPositions.zs};
    }

private:
    std::array<std::uint16_t, 2> jointHierarchy;
    dnac::RawVector3Vector neutralJointTranslations;
    dnac::RawVector3Vector vertexPositions;
};

class TranslateCommandTest : public ::testing::Test {
protected:
    void SetUp() override {
        TranslatableDNAReader fixtures;
        output = dnac::makeScoped<dnac::DNACalibDNAReader>(&fixtures);
        delta = {1.0f, 2.0f, 3.0f};

        float jxs[] = {2.0f, 2.5f};
        float jys[] = {5.0f, 4.5f};
        float jzs[] = {7.0f, 8.0f};
        expectedNeutralJointTranslationXs.assign(jxs, jxs + 2ul);
        expectedNeutralJointTranslationYs.assign(jys, jys + 2ul);
        expectedNeutralJointTranslationZs.assign(jzs, jzs + 2ul);

        float vxs[] = {5.0f, 13.0f, 24.5f, -3.0f, 3.0f};
        float vys[] = {3.0f, 4.0f, 5.0f, 6.0f, 7.0f};
        float vzs[] = {14.0f, -2.5f, 25.0f, 6.0f, 9.1f};
        expectedVertexPositionXs.assign(vxs, vxs + 5ul);
        expectedVertexPositionYs.assign(vys, vys + 5ul);
        expectedVertexPositionZs.assign(vzs, vzs + 5ul);
    }

    void TearDown() override {
    }

protected:
    dnac::ScopedPtr<dnac::DNACalibDNAReader, dnac::FactoryDestroy<dnac::DNACalibDNAReader>> output;

    dnac::Vector3 delta;

    dnac::Vector<float> expectedNeutralJointTranslationXs;
    dnac::Vector<float> expectedNeutralJointTranslationYs;
    dnac::Vector<float> expectedNeutralJointTranslationZs;
    dnac::Vector<float> expectedVertexPositionXs;
    dnac::Vector<float> expectedVertexPositionYs;
    dnac::Vector<float> expectedVertexPositionZs;
};

}  // namespace

TEST_F(TranslateCommandTest, AddDelta) {
    dnac::TranslateCommand cmd(delta);
    cmd.run(output.get());

    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationXs(),
                         expectedNeutralJointTranslationXs,
                         expectedNeutralJointTranslationXs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationYs(),
                         expectedNeutralJointTranslationYs,
                         expectedNeutralJointTranslationYs.size(),
                         0.0001);
    ASSERT_ELEMENTS_NEAR(output->getNeutralJointTranslationZs(),
                         expectedNeutralJointTranslationZs,
                         expectedNeutralJointTranslationZs.size(),
                         0.0001);

    ASSERT_ELEMENTS_NEAR(output->getVertexPositionXs(0), expectedVertexPositionXs, expectedVertexPositionXs.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionYs(0), expectedVertexPositionYs, expectedVertexPositionYs.size(), 0.0001);
    ASSERT_ELEMENTS_NEAR(output->getVertexPositionZs(0), expectedVertexPositionZs, expectedVertexPositionZs.size(), 0.0001);
}
