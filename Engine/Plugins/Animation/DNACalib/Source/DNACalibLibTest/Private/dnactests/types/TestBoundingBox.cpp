// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/types/BoundingBox.h"

#include <array>

namespace {

class BoundingBoxTest : public ::testing::Test {
protected:
    void SetUp() override {
        figureA = {tdm::fvec2{-34.2f, 15.0f},
                   tdm::fvec2{0.0f, 0.0f},
                   tdm::fvec2{16.2f, -2.0f},
                   tdm::fvec2{11.0f, 3.0f},
                   tdm::fvec2{10.0f, -30.0f}};

        figureB = {tdm::fvec2{1.0f, 0.0f}, tdm::fvec2{0.0f, 1.0f}, tdm::fvec2{-1.0f, 0.0f}, tdm::fvec2{0.0f, -1.0f}};
    }

    void TearDown() override {
    }

protected:
    std::array<tdm::fvec2, 5> figureA;
    dnac::Vector<tdm::fvec2> figureB;
};

}  // namespace

TEST_F(BoundingBoxTest, Constructor) {
    dnac::BoundingBox aBB{figureA, 0.00001f};
    ASSERT_NEAR(aBB.getMin()[0], -34.2f, 0.0001f);
    ASSERT_NEAR(aBB.getMin()[1], -30.0f, 0.0001f);
    ASSERT_NEAR(aBB.getMax()[0], 16.2f, 0.0001f);
    ASSERT_NEAR(aBB.getMax()[1], 15.0f, 0.0001f);

    dnac::BoundingBox bBB{figureB, 0.00001f};
    ASSERT_NEAR(bBB.getMin()[0], -1.0f, 0.0001f);
    ASSERT_NEAR(bBB.getMin()[1], -1.0f, 0.0001f);
    ASSERT_NEAR(bBB.getMax()[0], 1.0f, 0.0001f);
    ASSERT_NEAR(bBB.getMax()[1], 1.0f, 0.0001f);
}

TEST_F(BoundingBoxTest, Contains) {
    dnac::BoundingBox aBB{figureA};
    ASSERT_TRUE(aBB.contains({-34.200001f, 15.00001f}));
    ASSERT_FALSE(aBB.contains({-35.2f, 2.0f}));
    ASSERT_TRUE(aBB.contains({1.0f, 1.0f}));

    dnac::BoundingBox bBB{figureB};
    ASSERT_TRUE(bBB.contains({-1.0f, -1.0f}));
    ASSERT_FALSE(bBB.contains({0.0f, 2.0f}));
    ASSERT_TRUE(bBB.contains({0.0f, 0.0f}));
}
