// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/types/Triangle.h"

namespace {

class TriangleTest : public ::testing::Test {
protected:
    void SetUp() override {
        verticesA = {tdm::fvec2{1.5f, 10.0f}, tdm::fvec2{3.0f, 5.0f}, tdm::fvec2{-5.0f, -2.0f}};

        verticesB = {tdm::fvec2{12.5f, 13.5f}, tdm::fvec2{20.0f, 0.0f}, tdm::fvec2{0.0f, 0.0f}};

        point0 = {0.0f, 4.0f};
        point1 = {4.0f, 0.0f};
        expectedBarycentricAPoint0 = {0.2574f, 0.4158f, 0.3267f};
        expectedBarycentricAPoint1 = {-0.9306f, 1.8811f, 0.0495f};
        expectedBarycentricBPoint0 = {0.2962f, -0.1851f, 0.8888f};
        expectedBarycentricBPoint1 = {0.0f, 0.2f, 0.8f};
    }

    void TearDown() override {
    }

protected:
    std::array<tdm::fvec2, 3> verticesA;
    std::array<tdm::fvec2, 3> verticesB;
    tdm::fvec2 point0;
    tdm::fvec2 point1;
    tdm::fvec3 expectedBarycentricAPoint0;
    tdm::fvec3 expectedBarycentricAPoint1;
    tdm::fvec3 expectedBarycentricBPoint0;
    tdm::fvec3 expectedBarycentricBPoint1;
};

}  // namespace

TEST_F(TriangleTest, Constructor) {
    dnac::Triangle triangleA{verticesA};
    ASSERT_ELEMENTS_NEAR(triangleA.A(), verticesA[0], 2u, 0.0001f)
    ASSERT_ELEMENTS_NEAR(triangleA.B(), verticesA[1], 2u, 0.0001f)
    ASSERT_ELEMENTS_NEAR(triangleA.C(), verticesA[2], 2u, 0.0001f)

    dnac::Triangle triangleB{verticesB};
    ASSERT_ELEMENTS_NEAR(triangleB.A(), verticesB[0], 2u, 0.0001f)
    ASSERT_ELEMENTS_NEAR(triangleB.B(), verticesB[1], 2u, 0.0001f)
    ASSERT_ELEMENTS_NEAR(triangleB.C(), verticesB[2], 2u, 0.0001f)
}

TEST_F(TriangleTest, GetBarycentricCoords) {
    dnac::Triangle triangleA{verticesA};
    ASSERT_ELEMENTS_NEAR(triangleA.getBarycentricCoords(point0), expectedBarycentricAPoint0, 3u, 0.0001f);
    ASSERT_ELEMENTS_NEAR(triangleA.getBarycentricCoords(point1), expectedBarycentricAPoint1, 3u, 0.0001f);

    tdm::fvec3 expectedBarycentricVertexA{1.0f, 0.0f, 0.0f};
    ASSERT_ELEMENTS_NEAR(triangleA.getBarycentricCoords(verticesA[0]), expectedBarycentricVertexA, 3u, 0.0001f);

    dnac::Triangle triangleB{verticesB};
    ASSERT_ELEMENTS_NEAR(triangleB.getBarycentricCoords(point0), expectedBarycentricBPoint0, 3u, 0.0001f);
    ASSERT_ELEMENTS_NEAR(triangleB.getBarycentricCoords(point1), expectedBarycentricBPoint1, 3u, 0.0001f);
}
