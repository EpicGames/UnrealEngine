// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnactests/Defs.h"

#include "dnacalib/TypeDefs.h"
#include "dnacalib/types/UVBarycentricMapping.h"

namespace {

class UVBarycentricMappingTest : public ::testing::Test {
protected:
    void SetUp() override {
        Us = {1.5f, 3.0f, -5.0f, 12.5f, 20.0f, 0.0f};
        Vs = {10.0f, 5.0f, -2.0f, 13.5f, 0.0f, 0.0f};
        textureCoordinateUVIndices = {0u, 1u, 2u, 3u, 4u, 5u};
        vertexPositionIndices = {1u, 0u, 3u, 4u, 2u, 5u};

        expectedTriangle0VertexPositionIndices = {1u, 0u, 3u};
        expectedTriangle1VertexPositionIndices = {4u, 2u, 5u};

        const float alpha = dnac::BoundingBox::defaultAlpha;
        expectedBoundingBox0Min = tdm::fvec2{-5.0f, -2.0f} - alpha;
        expectedBoundingBox0Max = tdm::fvec2{3.0f, 10.0f} + alpha;
        expectedBoundingBox1Min = tdm::fvec2{0.0f, 0.0f} - alpha;
        expectedBoundingBox1Max = tdm::fvec2{20.0f, 13.5f} + alpha;

        point0 = {0.0f, 4.0f};
        point1 = {4.0f, 1.0f};
        expectedBarycentricPoint0 = {0.2574f, 0.4158f, 0.3267f};
        expectedBarycentricPoint1 = {0.0740f, 0.1537f, 0.7722f};
        expectedBarycentricPoint2 = {0.0f, 0.0f, 0.0f};

        faces = {std::array<std::uint32_t, 3>{0u, 1u, 2u}, std::array<std::uint32_t, 3>{3u, 4u, 5u}};

        faceGetter = [this](std::uint32_t faceIndex) {
            if (faceIndex == 0u) {
                return dnac::ConstArrayView<uint32_t>{faces[0]};
            }
            return dnac::ConstArrayView<uint32_t>{faces[1]};
        };
    }

    void TearDown() override {
    }

protected:
    std::array<float, 6> Us;
    std::array<float, 6> Vs;
    std::array<uint32_t, 6> vertexPositionIndices;
    std::array<uint32_t, 6> textureCoordinateUVIndices;
    std::function<dnac::ConstArrayView<std::uint32_t>(std::uint32_t)> faceGetter;
    std::uint32_t faceCount = 2u;
    std::array<std::array<std::uint32_t, 3>, 2> faces;

    std::array<uint32_t, 3> expectedTriangle0VertexPositionIndices;
    std::array<uint32_t, 3> expectedTriangle1VertexPositionIndices;

    tdm::fvec2 expectedBoundingBox0Min;
    tdm::fvec2 expectedBoundingBox0Max;
    tdm::fvec2 expectedBoundingBox1Min;
    tdm::fvec2 expectedBoundingBox1Max;

    tdm::fvec2 point0;
    tdm::fvec2 point1;
    tdm::fvec3 expectedBarycentricPoint0;
    tdm::fvec3 expectedBarycentricPoint1;
    tdm::fvec3 expectedBarycentricPoint2;
};

}  // namespace

TEST_F(UVBarycentricMappingTest, Constructor) {
    dnac::UVBarycentricMapping mapping{faceGetter,
                                       dnac::ConstArrayView<std::uint32_t>{vertexPositionIndices},
                                       dnac::ConstArrayView<std::uint32_t>{textureCoordinateUVIndices},
                                       dnac::ConstArrayView<float>{Us},
                                       dnac::ConstArrayView<float>{Vs},
                                       faceCount,
                                       nullptr};
    auto triangle = mapping.getTriangle(0);
    ASSERT_EQ(triangle.A(), tdm::fvec2(Us[0], Vs[0]));
    ASSERT_EQ(triangle.B(), tdm::fvec2(Us[1], Vs[1]));
    ASSERT_EQ(triangle.C(), tdm::fvec2(Us[2], Vs[2]));

    auto vertexPositions = mapping.getTrianglePositionIndices(0);
    ASSERT_ELEMENTS_EQ(vertexPositions, expectedTriangle0VertexPositionIndices, 3u);

    triangle = mapping.getTriangle(1);
    ASSERT_EQ(triangle.A(), tdm::fvec2(Us[3], Vs[3]));
    ASSERT_EQ(triangle.B(), tdm::fvec2(Us[4], Vs[4]));
    ASSERT_EQ(triangle.C(), tdm::fvec2(Us[5], Vs[5]));

    vertexPositions = mapping.getTrianglePositionIndices(1);
    ASSERT_ELEMENTS_EQ(vertexPositions, expectedTriangle1VertexPositionIndices, 3u);

    auto boundingBoxes = mapping.getBoundingBoxes();
    ASSERT_EQ(boundingBoxes[0].getMin(), expectedBoundingBox0Min);
    ASSERT_EQ(boundingBoxes[0].getMax(), expectedBoundingBox0Max);

    ASSERT_EQ(boundingBoxes[1].getMin(), expectedBoundingBox1Min);
    ASSERT_EQ(boundingBoxes[1].getMax(), expectedBoundingBox1Max);
}

TEST_F(UVBarycentricMappingTest, GetBarycentric) {
    dnac::UVBarycentricMapping mapping{faceGetter,
                                       dnac::ConstArrayView<std::uint32_t>{vertexPositionIndices},
                                       dnac::ConstArrayView<std::uint32_t>{textureCoordinateUVIndices},
                                       dnac::ConstArrayView<float>{Us},
                                       dnac::ConstArrayView<float>{Vs},
                                       faceCount,
                                       nullptr};

    auto barycentricPositionIndicesPair = mapping.getBarycentric(point0);

    auto barycentric = std::get<0>(barycentricPositionIndicesPair);
    ASSERT_ELEMENTS_NEAR(barycentric, expectedBarycentricPoint0, 3u, 0.0001f);

    auto vertexPositions = std::get<1>(barycentricPositionIndicesPair);
    ASSERT_ELEMENTS_EQ(vertexPositions, expectedTriangle0VertexPositionIndices, 3u);

    barycentricPositionIndicesPair = mapping.getBarycentric(point1);

    barycentric = std::get<0>(barycentricPositionIndicesPair);
    ASSERT_ELEMENTS_NEAR(barycentric, expectedBarycentricPoint1, 3u, 0.0001f);

    vertexPositions = std::get<1>(barycentricPositionIndicesPair);
    ASSERT_ELEMENTS_EQ(vertexPositions, expectedTriangle1VertexPositionIndices, 3u);

    barycentricPositionIndicesPair = mapping.getBarycentric(tdm::fvec2{50.0f, 50.0f});

    barycentric = std::get<0>(barycentricPositionIndicesPair);
    ASSERT_ELEMENTS_NEAR(barycentric, expectedBarycentricPoint2, 3u, 0.0001f);

    vertexPositions = std::get<1>(barycentricPositionIndicesPair);
    ASSERT_EQ(vertexPositions.size(), 0u);
}
