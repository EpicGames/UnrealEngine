// Copyright Epic Games, Inc. All Rights Reserved.
#include "dnactests/Defs.h"

#include "dnacalib/utils/Algorithm.h"

#include <cstdint>

static tdm::fmat4 getTransformationMatrix(const tdm::fvec3& translation,
                                          const tdm::frad3& rotation,
                                          const tdm::fvec3& scale = {1.0f, 1.0f, 1.0f}) {
    const auto rotSigns = tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive};
    return tdm::scale(scale) * tdm::rotate(rotation, tdm::rot_seq::xyz, rotSigns) * tdm::translate(translation);
}

TEST(TestAlgorithm, ExtractTranslationMatrix) {
    using namespace tdm::ang_literals;
    tdm::fvec3 t{1.5f, 0.6f, -0.2f};
    auto transform = getTransformationMatrix(t, {1.0_frad, -2.0_frad, 3.5_frad});
    auto tActual = dnac::extractTranslationMatrix(transform);
    auto tExpected = tdm::translate(t);
    for (auto i = 0u; i < tActual.rows(); i++) {
        for (auto j = 0u; j < tActual.columns(); j++) {
            ASSERT_NEAR(tActual[i][j], tExpected[i][j], 0.0001f);
        }
    }
}

TEST(TestAlgorithm, ExtractRotationMatrix) {
    using namespace tdm::ang_literals;
    const auto rotSigns = tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive};
    tdm::frad3 r{1.5_frad, 0.6_frad, -0.2_frad};
    auto transform = getTransformationMatrix({1.0f, -2.0f, 3.5f}, r);
    auto rActual = dnac::extractRotationMatrix(transform);
    auto rExpected = tdm::rotate(r[0], r[1], r[2], tdm::rot_seq::xyz, rotSigns);
    for (auto i = 0u; i < rActual.rows(); i++) {
        for (auto j = 0u; j < rActual.columns(); j++) {
            ASSERT_NEAR(rActual[i][j], rExpected[i][j], 0.0001f);
        }
    }
}

TEST(TestAlgorithm, ExtractRotationVector) {
    using namespace tdm::ang_literals;
    const auto rotSigns = tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive};
    tdm::frad3 r{0.5_frad, 0.6_frad, 0.2_frad};
    auto transform = tdm::rotate(r[0], r[1], r[2], tdm::rot_seq::xyz, rotSigns);
    auto rActual = dnac::extractRotationVector(transform);
    for (std::size_t i = 0u; i < 3u; i++) {
        ASSERT_NEAR(r[i].value, rActual[i].value, 0.0001f);
    }
}

TEST(TestAlgorithm, ExtractRotationVectorEdgeCase) {
    using namespace tdm::ang_literals;
    tdm::frad3 r{tdm::frad{60.0_fdeg}, tdm::frad{90.0_fdeg}, tdm::frad{0.0_fdeg}};
    const auto rotSigns = tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive};
    auto transform = tdm::rotate(r[0], r[1], r[2], tdm::rot_seq::xyz, rotSigns);
    auto rActual = dnac::extractRotationVector(transform);
    for (std::size_t i = 0u; i < 3u; i++) {
        ASSERT_NEAR(r[i].value, rActual[i].value, 0.0001f);
    }
}

TEST(TestAlgorithm, ExtractTranslationVector) {
    using namespace tdm::ang_literals;
    tdm::fvec3 t{0.1f, -0.6f, 1.2f};
    auto transform = getTransformationMatrix(t, tdm::frad3{1.0_frad, -2.0_frad, 3.5_frad});
    auto tActual = dnac::extractTranslationVector(transform);
    for (std::size_t i = 0u; i < 3u; i++) {
        ASSERT_NEAR(t[i], tActual[i], 0.0001f);
    }
}
