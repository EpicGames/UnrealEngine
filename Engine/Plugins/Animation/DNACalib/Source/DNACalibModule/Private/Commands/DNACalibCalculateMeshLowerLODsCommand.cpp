// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibCalculateMeshLowerLODsCommand.h"

#include "DNACalibDNAReader.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/CalculateMeshLowerLODsCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibCalculateMeshLowerLODsCommand::Impl
{
public:
	Impl() : Command{new dnac::CalculateMeshLowerLODsCommand{FMemoryResource::Instance()}}
	{
	}

	void SetMeshIndex(uint16 MeshIndex)
	{
		Command->setMeshIndex(MeshIndex);
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::CalculateMeshLowerLODsCommand> Command;
};

FDNACalibCalculateMeshLowerLODsCommand::FDNACalibCalculateMeshLowerLODsCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibCalculateMeshLowerLODsCommand::FDNACalibCalculateMeshLowerLODsCommand(uint16 MeshIndex) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndex(MeshIndex);
}

FDNACalibCalculateMeshLowerLODsCommand::~FDNACalibCalculateMeshLowerLODsCommand() = default;
FDNACalibCalculateMeshLowerLODsCommand::FDNACalibCalculateMeshLowerLODsCommand(FDNACalibCalculateMeshLowerLODsCommand&&) = default;
FDNACalibCalculateMeshLowerLODsCommand& FDNACalibCalculateMeshLowerLODsCommand::operator=(FDNACalibCalculateMeshLowerLODsCommand&&) = default;

void FDNACalibCalculateMeshLowerLODsCommand::SetMeshIndex(uint16 MeshIndex)
{
	ImplPtr->SetMeshIndex(MeshIndex);
}

void FDNACalibCalculateMeshLowerLODsCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
