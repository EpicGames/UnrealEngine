// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibClearBlendShapesCommand.h"

#include "DNACalibDNAReader.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/ClearBlendShapesCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibClearBlendShapesCommand::Impl
{
public:
	Impl() : Command{new dnac::ClearBlendShapesCommand{FMemoryResource::Instance()}}
	{
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::ClearBlendShapesCommand> Command;
};

FDNACalibClearBlendShapesCommand::FDNACalibClearBlendShapesCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibClearBlendShapesCommand::~FDNACalibClearBlendShapesCommand() = default;
FDNACalibClearBlendShapesCommand::FDNACalibClearBlendShapesCommand(FDNACalibClearBlendShapesCommand&&) = default;
FDNACalibClearBlendShapesCommand& FDNACalibClearBlendShapesCommand::operator=(FDNACalibClearBlendShapesCommand&&) = default;

void FDNACalibClearBlendShapesCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
