// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibCommandSequence.h"

#include "DNACalibDNAReader.h"
#include "FMemoryResource.h"

class FDNACalibCommandSequence::Impl
{
public:
	void Add(IDNACalibCommand* Command)
	{
		Commands.Add(Command);
	}

	void Remove(IDNACalibCommand* Command)
	{
		Commands.Remove(Command);
	}

	bool Contains(IDNACalibCommand* Command) const
	{
		return Commands.Contains(Command);
	}

	size_t Size() const
	{
		return static_cast<size_t>(Commands.Num());
	}

	void Run(FDNACalibDNAReader* Output)
	{
		for (auto Cmd : Commands)
		{
			Cmd->Run(Output);
		}
	}

private:
	TArray<IDNACalibCommand*> Commands;
};

FDNACalibCommandSequence::FDNACalibCommandSequence() : ImplPtr{new Impl{}}
{
}

FDNACalibCommandSequence::~FDNACalibCommandSequence() = default;
FDNACalibCommandSequence::FDNACalibCommandSequence(FDNACalibCommandSequence&&) = default;
FDNACalibCommandSequence& FDNACalibCommandSequence::operator=(FDNACalibCommandSequence&&) = default;

void FDNACalibCommandSequence::Add(IDNACalibCommand* Command)
{
	ImplPtr->Add(Command);
}

void FDNACalibCommandSequence::Add(TArrayView<IDNACalibCommand> Commands)
{
	for (auto& Cmd : Commands)
	{
		ImplPtr->Add(&Cmd);
	}
}

void FDNACalibCommandSequence::Remove(IDNACalibCommand* Command)
{
	ImplPtr->Remove(Command);
}

void FDNACalibCommandSequence::Remove(TArrayView<IDNACalibCommand> Commands)
{
	for (auto& Cmd : Commands)
	{
		ImplPtr->Remove(&Cmd);
	}
}

bool FDNACalibCommandSequence::Contains(IDNACalibCommand* Command) const
{
	return ImplPtr->Contains(Command);
}

size_t FDNACalibCommandSequence::Size() const
{
	return ImplPtr->Size();
}

void FDNACalibCommandSequence::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
