// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibComputeVertexPositionDeltasCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "DNAReader.h"
#include "FMemoryResource.h"
#include "VecArray.h"

#include "dnacalib/commands/SetVertexPositionsCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibComputeVertexPositionDeltasCommand::Impl
{
public:
	Impl() : Command{new dnac::SetVertexPositionsCommand{FMemoryResource::Instance()}}
	{
	}

	void SetReaderA(IDNAReader* Reader)
	{
		ReaderA = Reader;
	}

	void SetReaderB(IDNAReader* Reader)
	{
		ReaderB = Reader;
	}

	void Run(FDNACalibDNAReader* Output)
	{
		for (uint16 MeshIndexPlusOne = ReaderA->GetMeshCount(); MeshIndexPlusOne > 0u; --MeshIndexPlusOne)
		{
			const auto MeshIndex = static_cast<uint16>(MeshIndexPlusOne - 1u);
			if (ReaderB != Output)
			{
				const auto RawReaderB = ReaderB->Unwrap();
				const auto BXs = RawReaderB->getVertexPositionXs(MeshIndex);
				const auto BYs = RawReaderB->getVertexPositionYs(MeshIndex);
				const auto BZs = RawReaderB->getVertexPositionZs(MeshIndex);
				Command->setMeshIndex(MeshIndex);
				Command->setPositions(BXs, BYs, BZs);
				Command->setOperation(dnac::VectorOperation::Interpolate);
				Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
			}
			const auto RawReaderA = ReaderA->Unwrap();
			const auto AXs = RawReaderA->getVertexPositionXs(MeshIndex);
			const auto AYs = RawReaderA->getVertexPositionYs(MeshIndex);
			const auto AZs = RawReaderA->getVertexPositionZs(MeshIndex);
			Command->setMeshIndex(MeshIndex);
			Command->setPositions(AXs, AYs, AZs);
			Command->setOperation(dnac::VectorOperation::Subtract);
			Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
		}
	}

private:
	TUniquePtr<dnac::SetVertexPositionsCommand> Command;
	IDNAReader* ReaderA;
	IDNAReader* ReaderB;
};

FDNACalibComputeVertexPositionDeltasCommand::FDNACalibComputeVertexPositionDeltasCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibComputeVertexPositionDeltasCommand::FDNACalibComputeVertexPositionDeltasCommand(IDNAReader* ReaderA, IDNAReader* ReaderB) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetReaderA(ReaderA);
	ImplPtr->SetReaderB(ReaderB);
}

FDNACalibComputeVertexPositionDeltasCommand::~FDNACalibComputeVertexPositionDeltasCommand() = default;
FDNACalibComputeVertexPositionDeltasCommand::FDNACalibComputeVertexPositionDeltasCommand(FDNACalibComputeVertexPositionDeltasCommand&&) = default;
FDNACalibComputeVertexPositionDeltasCommand& FDNACalibComputeVertexPositionDeltasCommand::operator=(FDNACalibComputeVertexPositionDeltasCommand&&) = default;

void FDNACalibComputeVertexPositionDeltasCommand::SetReaderA(IDNAReader* Reader)
{
	ImplPtr->SetReaderA(Reader);
}

void FDNACalibComputeVertexPositionDeltasCommand::SetReaderB(IDNAReader* Reader)
{
	ImplPtr->SetReaderB(Reader);
}

void FDNACalibComputeVertexPositionDeltasCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
