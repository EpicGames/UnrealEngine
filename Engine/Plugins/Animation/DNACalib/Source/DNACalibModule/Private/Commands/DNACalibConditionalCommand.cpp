// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibConditionalCommand.h"

#include "DNACalibDNAReader.h"

class FDNACalibConditionalCommand::Impl
{
private:
	using TCondition = TFunction<bool(IDNACalibCommand*, FDNACalibDNAReader*)>;

public:
	Impl() :
		Command{nullptr},
		Condition{}
	{
	}

	void SetCommand(IDNACalibCommand* NewCommand)
	{
		Command = NewCommand;
	}

	void SetCondition(TCondition NewCondition)
	{
		Condition = NewCondition;
	}

	void Run(FDNACalibDNAReader* Output)
	{
		if (Command && Condition && Condition(Command, Output))
		{
			Command->Run(Output);
		}
	}

private:
	IDNACalibCommand* Command;
	TCondition Condition;
};

FDNACalibConditionalCommand::FDNACalibConditionalCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibConditionalCommand::FDNACalibConditionalCommand(IDNACalibCommand* Command, TCondition Condition) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetCommand(Command);
	ImplPtr->SetCondition(Condition);
}

FDNACalibConditionalCommand::~FDNACalibConditionalCommand() = default;
FDNACalibConditionalCommand::FDNACalibConditionalCommand(FDNACalibConditionalCommand&&) = default;
FDNACalibConditionalCommand& FDNACalibConditionalCommand::operator=(FDNACalibConditionalCommand&&) = default;

void FDNACalibConditionalCommand::SetCommand(IDNACalibCommand* Command)
{
	ImplPtr->SetCommand(Command);
}

void FDNACalibConditionalCommand::SetCondition(TCondition Condition)
{
	ImplPtr->SetCondition(Condition);
}

void FDNACalibConditionalCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
