// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibPruneBlendShapeTargetsCommand.h"

#include "DNACalibDNAReader.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/PruneBlendShapeTargetsCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibPruneBlendShapeTargetsCommand::Impl
{
public:
	Impl() : Command{new dnac::PruneBlendShapeTargetsCommand{FMemoryResource::Instance()}}
	{
	}

	void SetThreshold(float Threshold)
	{
		Command->setThreshold(Threshold);
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::PruneBlendShapeTargetsCommand> Command;
};

FDNACalibPruneBlendShapeTargetsCommand::FDNACalibPruneBlendShapeTargetsCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibPruneBlendShapeTargetsCommand::FDNACalibPruneBlendShapeTargetsCommand(float Threshold) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetThreshold(Threshold);
}

FDNACalibPruneBlendShapeTargetsCommand::~FDNACalibPruneBlendShapeTargetsCommand() = default;
FDNACalibPruneBlendShapeTargetsCommand::FDNACalibPruneBlendShapeTargetsCommand(FDNACalibPruneBlendShapeTargetsCommand&&) = default;
FDNACalibPruneBlendShapeTargetsCommand& FDNACalibPruneBlendShapeTargetsCommand::operator=(FDNACalibPruneBlendShapeTargetsCommand&&) = default;

void FDNACalibPruneBlendShapeTargetsCommand::SetThreshold(float Threshold)
{
	ImplPtr->SetThreshold(Threshold);
}

void FDNACalibPruneBlendShapeTargetsCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
