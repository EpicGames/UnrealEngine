// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibRemoveAnimatedMapCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/RemoveAnimatedMapCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibRemoveAnimatedMapCommand::Impl
{
public:
	Impl() : Command{new dnac::RemoveAnimatedMapCommand{FMemoryResource::Instance()}}
	{
	}

	void SetAnimatedMapIndex(uint16 AnimatedMapIndex)
	{
		Command->setAnimatedMapIndex(AnimatedMapIndex);
	}

	void SetAnimatedMapIndices(TArrayView<uint16> AnimatedMapIndices)
	{
		Command->setAnimatedMapIndices(ViewOf(AnimatedMapIndices));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::RemoveAnimatedMapCommand> Command;
};

FDNACalibRemoveAnimatedMapCommand::FDNACalibRemoveAnimatedMapCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibRemoveAnimatedMapCommand::FDNACalibRemoveAnimatedMapCommand(uint16 AnimatedMapIndex) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetAnimatedMapIndex(AnimatedMapIndex);
}

FDNACalibRemoveAnimatedMapCommand::FDNACalibRemoveAnimatedMapCommand(TArrayView<uint16> AnimatedMapIndices) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetAnimatedMapIndices(AnimatedMapIndices);
}

FDNACalibRemoveAnimatedMapCommand::~FDNACalibRemoveAnimatedMapCommand() = default;
FDNACalibRemoveAnimatedMapCommand::FDNACalibRemoveAnimatedMapCommand(FDNACalibRemoveAnimatedMapCommand&&) = default;
FDNACalibRemoveAnimatedMapCommand& FDNACalibRemoveAnimatedMapCommand::operator=(FDNACalibRemoveAnimatedMapCommand&&) = default;

void FDNACalibRemoveAnimatedMapCommand::SetAnimatedMapIndex(uint16 AnimatedMapIndex)
{
	ImplPtr->SetAnimatedMapIndex(AnimatedMapIndex);
}

void FDNACalibRemoveAnimatedMapCommand::SetAnimatedMapIndices(TArrayView<uint16> AnimatedMapIndices)
{
	ImplPtr->SetAnimatedMapIndices(AnimatedMapIndices);
}

void FDNACalibRemoveAnimatedMapCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
