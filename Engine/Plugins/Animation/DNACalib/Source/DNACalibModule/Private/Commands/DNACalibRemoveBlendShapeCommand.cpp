// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibRemoveBlendShapeCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/RemoveBlendShapeCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibRemoveBlendShapeCommand::Impl
{
public:
	Impl() : Command{new dnac::RemoveBlendShapeCommand{FMemoryResource::Instance()}}
	{
	}

	void SetBlendShapeIndex(uint16 BlendShapeIndex)
	{
		Command->setBlendShapeIndex(BlendShapeIndex);
	}

	void SetBlendShapeIndices(TArrayView<uint16> BlendShapeIndices)
	{
		Command->setBlendShapeIndices(ViewOf(BlendShapeIndices));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::RemoveBlendShapeCommand> Command;
};

FDNACalibRemoveBlendShapeCommand::FDNACalibRemoveBlendShapeCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibRemoveBlendShapeCommand::FDNACalibRemoveBlendShapeCommand(uint16 BlendShapeIndex) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetBlendShapeIndex(BlendShapeIndex);
}

FDNACalibRemoveBlendShapeCommand::FDNACalibRemoveBlendShapeCommand(TArrayView<uint16> BlendShapeIndices) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetBlendShapeIndices(BlendShapeIndices);
}

FDNACalibRemoveBlendShapeCommand::~FDNACalibRemoveBlendShapeCommand() = default;
FDNACalibRemoveBlendShapeCommand::FDNACalibRemoveBlendShapeCommand(FDNACalibRemoveBlendShapeCommand&&) = default;
FDNACalibRemoveBlendShapeCommand& FDNACalibRemoveBlendShapeCommand::operator=(FDNACalibRemoveBlendShapeCommand&&) = default;

void FDNACalibRemoveBlendShapeCommand::SetBlendShapeIndex(uint16 BlendShapeIndex)
{
	ImplPtr->SetBlendShapeIndex(BlendShapeIndex);
}

void FDNACalibRemoveBlendShapeCommand::SetBlendShapeIndices(TArrayView<uint16> BlendShapeIndices)
{
	ImplPtr->SetBlendShapeIndices(BlendShapeIndices);
}

void FDNACalibRemoveBlendShapeCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
