// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibRemoveJointAnimationCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/RemoveJointAnimationCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibRemoveJointAnimationCommand::Impl
{
public:
	Impl() : Command{new dnac::RemoveJointAnimationCommand{FMemoryResource::Instance()}}
	{
	}

	void SetJointIndex(uint16 JointIndex)
	{
		Command->setJointIndex(JointIndex);
	}

	void SetJointIndices(TArrayView<uint16> JointIndices)
	{
		Command->setJointIndices(ViewOf(JointIndices));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::RemoveJointAnimationCommand> Command;
};

FDNACalibRemoveJointAnimationCommand::FDNACalibRemoveJointAnimationCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibRemoveJointAnimationCommand::FDNACalibRemoveJointAnimationCommand(uint16 JointIndex) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetJointIndex(JointIndex);
}

FDNACalibRemoveJointAnimationCommand::FDNACalibRemoveJointAnimationCommand(TArrayView<uint16> JointIndices) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetJointIndices(JointIndices);
}

FDNACalibRemoveJointAnimationCommand::~FDNACalibRemoveJointAnimationCommand() = default;
FDNACalibRemoveJointAnimationCommand::FDNACalibRemoveJointAnimationCommand(FDNACalibRemoveJointAnimationCommand&&) = default;
FDNACalibRemoveJointAnimationCommand& FDNACalibRemoveJointAnimationCommand::operator=(FDNACalibRemoveJointAnimationCommand&&) = default;

void FDNACalibRemoveJointAnimationCommand::SetJointIndex(uint16 JointIndex)
{
	ImplPtr->SetJointIndex(JointIndex);
}

void FDNACalibRemoveJointAnimationCommand::SetJointIndices(TArrayView<uint16> JointIndices)
{
	ImplPtr->SetJointIndices(JointIndices);
}

void FDNACalibRemoveJointAnimationCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
