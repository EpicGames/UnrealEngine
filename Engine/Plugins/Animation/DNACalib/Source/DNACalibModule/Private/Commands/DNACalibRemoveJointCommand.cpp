// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibRemoveJointCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/RemoveJointCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibRemoveJointCommand::Impl
{
public:
	Impl() : Command{new dnac::RemoveJointCommand{FMemoryResource::Instance()}}
	{
	}

	void SetJointIndex(uint16 JointIndex)
	{
		Command->setJointIndex(JointIndex);
	}

	void SetJointIndices(TArrayView<uint16> JointIndices)
	{
		Command->setJointIndices(ViewOf(JointIndices));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::RemoveJointCommand> Command;
};

FDNACalibRemoveJointCommand::FDNACalibRemoveJointCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibRemoveJointCommand::FDNACalibRemoveJointCommand(uint16 JointIndex) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetJointIndex(JointIndex);
}

FDNACalibRemoveJointCommand::FDNACalibRemoveJointCommand(TArrayView<uint16> JointIndices) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetJointIndices(JointIndices);
}

FDNACalibRemoveJointCommand::~FDNACalibRemoveJointCommand() = default;
FDNACalibRemoveJointCommand::FDNACalibRemoveJointCommand(FDNACalibRemoveJointCommand&&) = default;
FDNACalibRemoveJointCommand& FDNACalibRemoveJointCommand::operator=(FDNACalibRemoveJointCommand&&) = default;

void FDNACalibRemoveJointCommand::SetJointIndex(uint16 JointIndex)
{
	ImplPtr->SetJointIndex(JointIndex);
}

void FDNACalibRemoveJointCommand::SetJointIndices(TArrayView<uint16> JointIndices)
{
	ImplPtr->SetJointIndices(JointIndices);
}

void FDNACalibRemoveJointCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
