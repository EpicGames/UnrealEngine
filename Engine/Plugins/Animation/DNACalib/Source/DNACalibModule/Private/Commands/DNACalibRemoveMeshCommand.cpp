// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibRemoveMeshCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/RemoveMeshCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibRemoveMeshCommand::Impl
{
public:
	Impl() : Command{new dnac::RemoveMeshCommand{FMemoryResource::Instance()}}
	{
	}

	void SetMeshIndex(uint16 MeshIndex)
	{
		Command->setMeshIndex(MeshIndex);
	}

	void SetMeshIndices(TArrayView<uint16> MeshIndices)
	{
		Command->setMeshIndices(ViewOf(MeshIndices));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::RemoveMeshCommand> Command;
};

FDNACalibRemoveMeshCommand::FDNACalibRemoveMeshCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibRemoveMeshCommand::FDNACalibRemoveMeshCommand(uint16 MeshIndex) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndex(MeshIndex);
}

FDNACalibRemoveMeshCommand::FDNACalibRemoveMeshCommand(TArrayView<uint16> MeshIndices) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndices(MeshIndices);
}

FDNACalibRemoveMeshCommand::~FDNACalibRemoveMeshCommand() = default;
FDNACalibRemoveMeshCommand::FDNACalibRemoveMeshCommand(FDNACalibRemoveMeshCommand&&) = default;
FDNACalibRemoveMeshCommand& FDNACalibRemoveMeshCommand::operator=(FDNACalibRemoveMeshCommand&&) = default;

void FDNACalibRemoveMeshCommand::SetMeshIndex(uint16 MeshIndex)
{
	ImplPtr->SetMeshIndex(MeshIndex);
}

void FDNACalibRemoveMeshCommand::SetMeshIndices(TArrayView<uint16> MeshIndices)
{
	ImplPtr->SetMeshIndices(MeshIndices);
}

void FDNACalibRemoveMeshCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
