// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibRenameAnimatedMapCommand.h"

#include "DNACalibDNAReader.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/RenameAnimatedMapCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibRenameAnimatedMapCommand::Impl
{
public:
	Impl() : Command{new dnac::RenameAnimatedMapCommand{FMemoryResource::Instance()}}
	{
	}

	void SetName(uint16 AnimatedMapIndex, const FString& NewName)
	{
		Command->setName(AnimatedMapIndex, TCHAR_TO_ANSI(*NewName));
	}

	void SetName(const FString& OldName, const FString& NewName)
	{
		Command->setName(TCHAR_TO_ANSI(*OldName), TCHAR_TO_ANSI(*NewName));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::RenameAnimatedMapCommand> Command;
};

FDNACalibRenameAnimatedMapCommand::FDNACalibRenameAnimatedMapCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibRenameAnimatedMapCommand::FDNACalibRenameAnimatedMapCommand(uint16 AnimatedMapIndex, const FString& NewName) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetName(AnimatedMapIndex, NewName);
}

FDNACalibRenameAnimatedMapCommand::FDNACalibRenameAnimatedMapCommand(const FString& OldName, const FString& NewName) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetName(OldName, NewName);
}

FDNACalibRenameAnimatedMapCommand::~FDNACalibRenameAnimatedMapCommand() = default;
FDNACalibRenameAnimatedMapCommand::FDNACalibRenameAnimatedMapCommand(FDNACalibRenameAnimatedMapCommand&&) = default;
FDNACalibRenameAnimatedMapCommand& FDNACalibRenameAnimatedMapCommand::operator=(FDNACalibRenameAnimatedMapCommand&&) = default;

void FDNACalibRenameAnimatedMapCommand::SetName(uint16 AnimatedMapIndex, const FString& NewName)
{
	ImplPtr->SetName(AnimatedMapIndex, NewName);
}

void FDNACalibRenameAnimatedMapCommand::SetName(const FString& OldName, const FString& NewName)
{
	ImplPtr->SetName(OldName, NewName);
}

void FDNACalibRenameAnimatedMapCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
