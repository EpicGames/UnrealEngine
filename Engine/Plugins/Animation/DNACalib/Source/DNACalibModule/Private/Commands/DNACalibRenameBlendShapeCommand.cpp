// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibRenameBlendShapeCommand.h"

#include "DNACalibDNAReader.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/RenameBlendShapeCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibRenameBlendShapeCommand::Impl
{
public:
	Impl() : Command{new dnac::RenameBlendShapeCommand{FMemoryResource::Instance()}}
	{
	}

	void SetName(uint16 BlendShapeIndex, const FString& NewName)
	{
		Command->setName(BlendShapeIndex, TCHAR_TO_ANSI(*NewName));
	}

	void SetName(const FString& OldName, const FString& NewName)
	{
		Command->setName(TCHAR_TO_ANSI(*OldName), TCHAR_TO_ANSI(*NewName));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::RenameBlendShapeCommand> Command;
};

FDNACalibRenameBlendShapeCommand::FDNACalibRenameBlendShapeCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibRenameBlendShapeCommand::FDNACalibRenameBlendShapeCommand(uint16 BlendShapeIndex, const FString& NewName) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetName(BlendShapeIndex, NewName);
}

FDNACalibRenameBlendShapeCommand::FDNACalibRenameBlendShapeCommand(const FString& OldName, const FString& NewName) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetName(OldName, NewName);
}

FDNACalibRenameBlendShapeCommand::~FDNACalibRenameBlendShapeCommand() = default;
FDNACalibRenameBlendShapeCommand::FDNACalibRenameBlendShapeCommand(FDNACalibRenameBlendShapeCommand&&) = default;
FDNACalibRenameBlendShapeCommand& FDNACalibRenameBlendShapeCommand::operator=(FDNACalibRenameBlendShapeCommand&&) = default;

void FDNACalibRenameBlendShapeCommand::SetName(uint16 BlendShapeIndex, const FString& NewName)
{
	ImplPtr->SetName(BlendShapeIndex, NewName);
}

void FDNACalibRenameBlendShapeCommand::SetName(const FString& OldName, const FString& NewName)
{
	ImplPtr->SetName(OldName, NewName);
}

void FDNACalibRenameBlendShapeCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
