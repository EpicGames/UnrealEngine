// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibRenameJointCommand.h"

#include "DNACalibDNAReader.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/RenameJointCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibRenameJointCommand::Impl
{
public:
	Impl() : Command{new dnac::RenameJointCommand{FMemoryResource::Instance()}}
	{
	}

	void SetName(uint16 JointIndex, const FString& NewName)
	{
		Command->setName(JointIndex, TCHAR_TO_ANSI(*NewName));
	}

	void SetName(const FString& OldName, const FString& NewName)
	{
		Command->setName(TCHAR_TO_ANSI(*OldName), TCHAR_TO_ANSI(*NewName));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::RenameJointCommand> Command;
};

FDNACalibRenameJointCommand::FDNACalibRenameJointCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibRenameJointCommand::FDNACalibRenameJointCommand(uint16 JointIndex, const FString& NewName) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetName(JointIndex, NewName);
}

FDNACalibRenameJointCommand::FDNACalibRenameJointCommand(const FString& OldName, const FString& NewName) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetName(OldName, NewName);
}

FDNACalibRenameJointCommand::~FDNACalibRenameJointCommand() = default;
FDNACalibRenameJointCommand::FDNACalibRenameJointCommand(FDNACalibRenameJointCommand&&) = default;
FDNACalibRenameJointCommand& FDNACalibRenameJointCommand::operator=(FDNACalibRenameJointCommand&&) = default;

void FDNACalibRenameJointCommand::SetName(uint16 JointIndex, const FString& NewName)
{
	ImplPtr->SetName(JointIndex, NewName);
}

void FDNACalibRenameJointCommand::SetName(const FString& OldName, const FString& NewName)
{
	ImplPtr->SetName(OldName, NewName);
}

void FDNACalibRenameJointCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
