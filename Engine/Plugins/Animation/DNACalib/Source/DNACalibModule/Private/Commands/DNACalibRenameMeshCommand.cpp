// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibRenameMeshCommand.h"

#include "DNACalibDNAReader.h"
#include "FMemoryResource.h"

#include "dnacalib/commands/RenameMeshCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibRenameMeshCommand::Impl
{
public:
	Impl() : Command{new dnac::RenameMeshCommand{FMemoryResource::Instance()}}
	{
	}

	void SetName(uint16 MeshIndex, const FString& NewName)
	{
		Command->setName(MeshIndex, TCHAR_TO_ANSI(*NewName));
	}

	void SetName(const FString& OldName, const FString& NewName)
	{
		Command->setName(TCHAR_TO_ANSI(*OldName), TCHAR_TO_ANSI(*NewName));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::RenameMeshCommand> Command;
};

FDNACalibRenameMeshCommand::FDNACalibRenameMeshCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibRenameMeshCommand::FDNACalibRenameMeshCommand(uint16 MeshIndex, const FString& NewName) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetName(MeshIndex, NewName);
}

FDNACalibRenameMeshCommand::FDNACalibRenameMeshCommand(const FString& OldName, const FString& NewName) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetName(OldName, NewName);
}

FDNACalibRenameMeshCommand::~FDNACalibRenameMeshCommand() = default;
FDNACalibRenameMeshCommand::FDNACalibRenameMeshCommand(FDNACalibRenameMeshCommand&&) = default;
FDNACalibRenameMeshCommand& FDNACalibRenameMeshCommand::operator=(FDNACalibRenameMeshCommand&&) = default;

void FDNACalibRenameMeshCommand::SetName(uint16 MeshIndex, const FString& NewName)
{
	ImplPtr->SetName(MeshIndex, NewName);
}

void FDNACalibRenameMeshCommand::SetName(const FString& OldName, const FString& NewName)
{
	ImplPtr->SetName(OldName, NewName);
}

void FDNACalibRenameMeshCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
