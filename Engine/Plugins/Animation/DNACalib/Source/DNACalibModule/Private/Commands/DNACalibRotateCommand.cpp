// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibRotateCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"
#include "VecArray.h"

#include "dnacalib/commands/RotateCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibRotateCommand::Impl
{
public:
	Impl() : Command{new dnac::RotateCommand{FMemoryResource::Instance()}}
	{
	}

	void SetRotation(FVector Degrees)
	{
		Command->setRotation(dnac::Vector3{static_cast<float>(Degrees.X), static_cast<float>(Degrees.Y), static_cast<float>(Degrees.Z)});
	}

	void SetOrigin(FVector Origin)
	{
		Command->setOrigin(dnac::Vector3{static_cast<float>(Origin.X), static_cast<float>(Origin.Y), static_cast<float>(Origin.Z)});
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::RotateCommand> Command;
};

FDNACalibRotateCommand::FDNACalibRotateCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibRotateCommand::FDNACalibRotateCommand(FVector Degrees, FVector Origin) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetRotation(Degrees);
	ImplPtr->SetOrigin(Origin);
}

FDNACalibRotateCommand::~FDNACalibRotateCommand() = default;
FDNACalibRotateCommand::FDNACalibRotateCommand(FDNACalibRotateCommand&&) = default;
FDNACalibRotateCommand& FDNACalibRotateCommand::operator=(FDNACalibRotateCommand&&) = default;

void FDNACalibRotateCommand::SetRotation(FVector Degrees)
{
	ImplPtr->SetRotation(Degrees);
}

void FDNACalibRotateCommand::SetOrigin(FVector Origin)
{
	ImplPtr->SetOrigin(Origin);
}

void FDNACalibRotateCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
