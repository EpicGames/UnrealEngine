// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibScaleCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"
#include "VecArray.h"

#include "dnacalib/commands/ScaleCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibScaleCommand::Impl
{
public:
	Impl() : Command{new dnac::ScaleCommand{FMemoryResource::Instance()}}
	{
	}

	void SetScale(float Scale)
	{
		Command->setScale(Scale);
	}

	void SetOrigin(FVector Origin)
	{
		Command->setOrigin(dnac::Vector3{static_cast<float>(Origin.X), static_cast<float>(Origin.Y), static_cast<float>(Origin.Z)});
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::ScaleCommand> Command;
};

FDNACalibScaleCommand::FDNACalibScaleCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibScaleCommand::FDNACalibScaleCommand(float Scale, FVector Origin) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetScale(Scale);
	ImplPtr->SetOrigin(Origin);
}

FDNACalibScaleCommand::~FDNACalibScaleCommand() = default;
FDNACalibScaleCommand::FDNACalibScaleCommand(FDNACalibScaleCommand&&) = default;
FDNACalibScaleCommand& FDNACalibScaleCommand::operator=(FDNACalibScaleCommand&&) = default;

void FDNACalibScaleCommand::SetScale(float Scale)
{
	ImplPtr->SetScale(Scale);
}

void FDNACalibScaleCommand::SetOrigin(FVector Origin)
{
	ImplPtr->SetOrigin(Origin);
}

void FDNACalibScaleCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
