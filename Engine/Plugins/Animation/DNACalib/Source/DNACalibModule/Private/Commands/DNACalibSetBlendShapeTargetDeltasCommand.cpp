// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibSetBlendShapeTargetDeltasCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"
#include "VecArray.h"

#include "dnacalib/commands/SetBlendShapeTargetDeltasCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibSetBlendShapeTargetDeltasCommand::Impl
{
public:
	Impl() : Command{new dnac::SetBlendShapeTargetDeltasCommand{FMemoryResource::Instance()}}
	{
	}

	void SetMeshIndex(uint16 MeshIndex)
	{
		Command->setMeshIndex(MeshIndex);
	}

	void SetBlendShapeTargetIndex(uint16 BlendShapeTargetIndex)
	{
		Command->setBlendShapeTargetIndex(BlendShapeTargetIndex);
	}

	void SetDeltas(TArrayView<const FVector> Deltas)
	{
		UnpackedDeltas.Assign(Deltas);
		Command->setDeltas(ViewOf(UnpackedDeltas.Xs), ViewOf(UnpackedDeltas.Ys), ViewOf(UnpackedDeltas.Zs));
	}

	void SetDeltas(TArrayView<const float> DXs, TArrayView<const float> DYs, TArrayView<const float> DZs)
	{
		Command->setDeltas(ViewOf(DXs), ViewOf(DYs), ViewOf(DZs));
	}

	void SetVertexIndices(TArrayView<const uint32> VertexIndices)
	{
		Command->setVertexIndices(ViewOf(VertexIndices));
	}

	void SetMasks(TArrayView<const float> Masks)
	{
		Command->setMasks(ViewOf(Masks));
	}

	void SetOperation(EDNACalibVectorOperation Operation)
	{
		Command->setOperation(static_cast<dnac::VectorOperation>(Operation));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::SetBlendShapeTargetDeltasCommand> Command;
	FVecArray UnpackedDeltas;
};

FDNACalibSetBlendShapeTargetDeltasCommand::FDNACalibSetBlendShapeTargetDeltasCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibSetBlendShapeTargetDeltasCommand::FDNACalibSetBlendShapeTargetDeltasCommand(uint16 MeshIndex, uint16 BlendShapeTargetIndex, TArrayView<const FVector> Deltas, TArrayView<const uint32> VertexIndices, EDNACalibVectorOperation Operation) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndex(MeshIndex);
	ImplPtr->SetBlendShapeTargetIndex(BlendShapeTargetIndex);
	ImplPtr->SetDeltas(Deltas);
	ImplPtr->SetVertexIndices(VertexIndices);
	ImplPtr->SetOperation(Operation);
}

FDNACalibSetBlendShapeTargetDeltasCommand::FDNACalibSetBlendShapeTargetDeltasCommand(uint16 MeshIndex, uint16 BlendShapeTargetIndex, TArrayView<const float> DXs, TArrayView<const float> DYs, TArrayView<const float> DZs, TArrayView<const uint32> VertexIndices, EDNACalibVectorOperation Operation) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndex(MeshIndex);
	ImplPtr->SetBlendShapeTargetIndex(BlendShapeTargetIndex);
	ImplPtr->SetDeltas(DXs, DYs, DZs);
	ImplPtr->SetVertexIndices(VertexIndices);
	ImplPtr->SetOperation(Operation);
}

FDNACalibSetBlendShapeTargetDeltasCommand::FDNACalibSetBlendShapeTargetDeltasCommand(uint16 MeshIndex, uint16 BlendShapeTargetIndex, TArrayView<const FVector> Deltas, TArrayView<const uint32> VertexIndices, TArrayView<const float> Masks, EDNACalibVectorOperation Operation) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndex(MeshIndex);
	ImplPtr->SetBlendShapeTargetIndex(BlendShapeTargetIndex);
	ImplPtr->SetDeltas(Deltas);
	ImplPtr->SetVertexIndices(VertexIndices);
	ImplPtr->SetMasks(Masks);
	ImplPtr->SetOperation(Operation);
}

FDNACalibSetBlendShapeTargetDeltasCommand::FDNACalibSetBlendShapeTargetDeltasCommand(uint16 MeshIndex, uint16 BlendShapeTargetIndex, TArrayView<const float> DXs, TArrayView<const float> DYs, TArrayView<const float> DZs, TArrayView<const uint32> VertexIndices, TArrayView<const float> Masks, EDNACalibVectorOperation Operation) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndex(MeshIndex);
	ImplPtr->SetBlendShapeTargetIndex(BlendShapeTargetIndex);
	ImplPtr->SetDeltas(DXs, DYs, DZs);
	ImplPtr->SetVertexIndices(VertexIndices);
	ImplPtr->SetMasks(Masks);
	ImplPtr->SetOperation(Operation);
}

FDNACalibSetBlendShapeTargetDeltasCommand::~FDNACalibSetBlendShapeTargetDeltasCommand() = default;
FDNACalibSetBlendShapeTargetDeltasCommand::FDNACalibSetBlendShapeTargetDeltasCommand(FDNACalibSetBlendShapeTargetDeltasCommand&&) = default;
FDNACalibSetBlendShapeTargetDeltasCommand& FDNACalibSetBlendShapeTargetDeltasCommand::operator=(FDNACalibSetBlendShapeTargetDeltasCommand&&) = default;

void FDNACalibSetBlendShapeTargetDeltasCommand::SetMeshIndex(uint16 MeshIndex)
{
	ImplPtr->SetMeshIndex(MeshIndex);
}

void FDNACalibSetBlendShapeTargetDeltasCommand::SetBlendShapeTargetIndex(uint16 BlendShapeTargetIndex)
{
	ImplPtr->SetBlendShapeTargetIndex(BlendShapeTargetIndex);
}

void FDNACalibSetBlendShapeTargetDeltasCommand::SetDeltas(TArrayView<const FVector> Deltas)
{
	ImplPtr->SetDeltas(Deltas);
}

void FDNACalibSetBlendShapeTargetDeltasCommand::SetDeltas(TArrayView<const float> DXs, TArrayView<const float> DYs, TArrayView<const float> DZs)
{
	ImplPtr->SetDeltas(DXs, DYs, DZs);
}

void FDNACalibSetBlendShapeTargetDeltasCommand::SetVertexIndices(TArrayView<const uint32> VertexIndices)
{
	ImplPtr->SetVertexIndices(VertexIndices);
}

void FDNACalibSetBlendShapeTargetDeltasCommand::SetMasks(TArrayView<const float> Masks)
{
	ImplPtr->SetMasks(Masks);
}

void FDNACalibSetBlendShapeTargetDeltasCommand::SetOperation(EDNACalibVectorOperation Operation)
{
	ImplPtr->SetOperation(Operation);
}

void FDNACalibSetBlendShapeTargetDeltasCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
