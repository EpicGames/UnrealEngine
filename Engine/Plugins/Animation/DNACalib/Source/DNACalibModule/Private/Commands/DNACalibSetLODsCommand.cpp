// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibSetLODsCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"
#include "VecArray.h"

#include "dnacalib/commands/SetLODsCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibSetLODsCommand::Impl
{
public:
	Impl() : Command{new dnac::SetLODsCommand{FMemoryResource::Instance()}}
	{
	}

	void SetLODs(TArrayView<const uint16> LODs)
	{
		Command->setLODs(ViewOf(LODs));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::SetLODsCommand> Command;
};

FDNACalibSetLODsCommand::FDNACalibSetLODsCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibSetLODsCommand::FDNACalibSetLODsCommand(TArrayView<const uint16> LODs) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetLODs(LODs);
}

FDNACalibSetLODsCommand::~FDNACalibSetLODsCommand() = default;
FDNACalibSetLODsCommand::FDNACalibSetLODsCommand(FDNACalibSetLODsCommand&&) = default;
FDNACalibSetLODsCommand& FDNACalibSetLODsCommand::operator=(FDNACalibSetLODsCommand&&) = default;

void FDNACalibSetLODsCommand::SetLODs(TArrayView<const uint16> LODs)
{
	ImplPtr->SetLODs(LODs);
}

void FDNACalibSetLODsCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
