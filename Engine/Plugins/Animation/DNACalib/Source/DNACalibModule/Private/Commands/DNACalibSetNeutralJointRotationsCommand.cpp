// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibSetNeutralJointRotationsCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"
#include "VecArray.h"

#include "dnacalib/commands/SetNeutralJointRotationsCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibSetNeutralJointRotationsCommand::Impl
{
public:
	Impl() : Command{new dnac::SetNeutralJointRotationsCommand{FMemoryResource::Instance()}}
	{
	}

	void SetRotations(TArrayView<const FVector> Rotations)
	{
		UnpackedRotations.Empty();
		UnpackedRotations.Reserve(Rotations.Num());
		for (const auto& Vec : Rotations)
		{
			UnpackedRotations.Xs.Add(Vec.X);
			UnpackedRotations.Ys.Add(Vec.Y);
			UnpackedRotations.Zs.Add(Vec.Z);
		}
		Command->setRotations(ViewOf(UnpackedRotations.Xs), ViewOf(UnpackedRotations.Ys), ViewOf(UnpackedRotations.Zs));
	}

	void SetRotations(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs)
	{
		check(Xs.Num() == Ys.Num() && Ys.Num() == Zs.Num());
		Command->setRotations(ViewOf(Xs), ViewOf(Ys), ViewOf(Zs));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::SetNeutralJointRotationsCommand> Command;
	FVecArray UnpackedRotations;
};

FDNACalibSetNeutralJointRotationsCommand::FDNACalibSetNeutralJointRotationsCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibSetNeutralJointRotationsCommand::FDNACalibSetNeutralJointRotationsCommand(TArrayView<const FVector> Rotations) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetRotations(Rotations);
}

FDNACalibSetNeutralJointRotationsCommand::FDNACalibSetNeutralJointRotationsCommand(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetRotations(Xs, Ys, Zs);
}

FDNACalibSetNeutralJointRotationsCommand::~FDNACalibSetNeutralJointRotationsCommand() = default;
FDNACalibSetNeutralJointRotationsCommand::FDNACalibSetNeutralJointRotationsCommand(FDNACalibSetNeutralJointRotationsCommand&&) = default;
FDNACalibSetNeutralJointRotationsCommand& FDNACalibSetNeutralJointRotationsCommand::operator=(FDNACalibSetNeutralJointRotationsCommand&&) = default;

void FDNACalibSetNeutralJointRotationsCommand::SetRotations(TArrayView<const FVector> Rotations)
{
	ImplPtr->SetRotations(Rotations);
}

void FDNACalibSetNeutralJointRotationsCommand::SetRotations(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs)
{
	ImplPtr->SetRotations(Xs, Ys, Zs);
}

void FDNACalibSetNeutralJointRotationsCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
