// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibSetNeutralJointTranslationsCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"
#include "VecArray.h"

#include "dnacalib/commands/SetNeutralJointTranslationsCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibSetNeutralJointTranslationsCommand::Impl
{
public:
	Impl() : Command{new dnac::SetNeutralJointTranslationsCommand{FMemoryResource::Instance()}}
	{
	}

	void SetTranslations(TArrayView<const FVector> Translations)
	{
		UnpackedTranslations.Empty();
		UnpackedTranslations.Reserve(Translations.Num());
		for (const auto& Vec : Translations)
		{
			UnpackedTranslations.Xs.Add(Vec.X);
			UnpackedTranslations.Ys.Add(Vec.Y);
			UnpackedTranslations.Zs.Add(Vec.Z);
		}
		Command->setTranslations(ViewOf(UnpackedTranslations.Xs), ViewOf(UnpackedTranslations.Ys), ViewOf(UnpackedTranslations.Zs));
	}

	void SetTranslations(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs)
	{
		check(Xs.Num() == Ys.Num() && Ys.Num() == Zs.Num());
		Command->setTranslations(ViewOf(Xs), ViewOf(Ys), ViewOf(Zs));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::SetNeutralJointTranslationsCommand> Command;
	FVecArray UnpackedTranslations;
};

FDNACalibSetNeutralJointTranslationsCommand::FDNACalibSetNeutralJointTranslationsCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibSetNeutralJointTranslationsCommand::FDNACalibSetNeutralJointTranslationsCommand(TArrayView<const FVector> Translations) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetTranslations(Translations);
}

FDNACalibSetNeutralJointTranslationsCommand::FDNACalibSetNeutralJointTranslationsCommand(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetTranslations(Xs, Ys, Zs);
}

FDNACalibSetNeutralJointTranslationsCommand::~FDNACalibSetNeutralJointTranslationsCommand() = default;
FDNACalibSetNeutralJointTranslationsCommand::FDNACalibSetNeutralJointTranslationsCommand(FDNACalibSetNeutralJointTranslationsCommand&&) = default;
FDNACalibSetNeutralJointTranslationsCommand& FDNACalibSetNeutralJointTranslationsCommand::operator=(FDNACalibSetNeutralJointTranslationsCommand&&) = default;

void FDNACalibSetNeutralJointTranslationsCommand::SetTranslations(TArrayView<const FVector> Translations)
{
	ImplPtr->SetTranslations(Translations);
}

void FDNACalibSetNeutralJointTranslationsCommand::SetTranslations(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs)
{
	ImplPtr->SetTranslations(Xs, Ys, Zs);
}

void FDNACalibSetNeutralJointTranslationsCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
