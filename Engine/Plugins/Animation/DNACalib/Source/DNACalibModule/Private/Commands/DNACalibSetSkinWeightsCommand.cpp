// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibSetSkinWeightsCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"
#include "VecArray.h"

#include "dnacalib/commands/SetSkinWeightsCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibSetSkinWeightsCommand::Impl
{
public:
	Impl() : Command{new dnac::SetSkinWeightsCommand{FMemoryResource::Instance()}}
	{
	}

	void SetMeshIndex(uint16 MeshIndex)
	{
		Command->setMeshIndex(MeshIndex);
	}

	void SetVertexIndex(uint32 VertexIndex)
	{
		Command->setVertexIndex(VertexIndex);
	}

	void SetWeights(TArrayView<const float> Weights)
	{
		Command->setWeights(ViewOf(Weights));
	}

	void SetJointIndices(TArrayView<const uint16> JointIndices)
	{
		Command->setJointIndices(ViewOf(JointIndices));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::SetSkinWeightsCommand> Command;
};

FDNACalibSetSkinWeightsCommand::FDNACalibSetSkinWeightsCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibSetSkinWeightsCommand::FDNACalibSetSkinWeightsCommand(uint16 MeshIndex, uint32 VertexIndex, TArrayView<const float> Weights, TArrayView<const uint16> JointIndices) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndex(MeshIndex);
	ImplPtr->SetVertexIndex(VertexIndex);
	ImplPtr->SetWeights(Weights);
	ImplPtr->SetJointIndices(JointIndices);
}

FDNACalibSetSkinWeightsCommand::~FDNACalibSetSkinWeightsCommand() = default;
FDNACalibSetSkinWeightsCommand::FDNACalibSetSkinWeightsCommand(FDNACalibSetSkinWeightsCommand&&) = default;
FDNACalibSetSkinWeightsCommand& FDNACalibSetSkinWeightsCommand::operator=(FDNACalibSetSkinWeightsCommand&&) = default;

void FDNACalibSetSkinWeightsCommand::SetMeshIndex(uint16 MeshIndex)
{
	ImplPtr->SetMeshIndex(MeshIndex);
}

void FDNACalibSetSkinWeightsCommand::SetVertexIndex(uint32 VertexIndex)
{
	ImplPtr->SetVertexIndex(VertexIndex);
}

void FDNACalibSetSkinWeightsCommand::SetWeights(TArrayView<const float> Weights)
{
	ImplPtr->SetWeights(Weights);
}

void FDNACalibSetSkinWeightsCommand::SetJointIndices(TArrayView<const uint16> JointIndices)
{
	ImplPtr->SetJointIndices(JointIndices);
}

void FDNACalibSetSkinWeightsCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
