// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibSetVertexPositionsCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"
#include "VecArray.h"

#include "dnacalib/commands/SetVertexPositionsCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibSetVertexPositionsCommand::Impl
{
public:
	Impl() : Command{new dnac::SetVertexPositionsCommand{FMemoryResource::Instance()}}
	{
	}

	void SetMeshIndex(uint16 MeshIndex)
	{
		Command->setMeshIndex(MeshIndex);
	}

	void SetPositions(TArrayView<const FVector> Positions)
	{
		UnpackedPositions.Assign(Positions);
		Command->setPositions(ViewOf(UnpackedPositions.Xs), ViewOf(UnpackedPositions.Ys), ViewOf(UnpackedPositions.Zs));
	}

	void SetPositions(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs)
	{
		Command->setPositions(ViewOf(Xs), ViewOf(Ys), ViewOf(Zs));
	}

	void SetMasks(TArrayView<const float> Masks)
	{
		Command->setMasks(ViewOf(Masks));
	}

	void SetOperation(EDNACalibVectorOperation Operation)
	{
		Command->setOperation(static_cast<dnac::VectorOperation>(Operation));
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::SetVertexPositionsCommand> Command;
	FVecArray UnpackedPositions;
};

FDNACalibSetVertexPositionsCommand::FDNACalibSetVertexPositionsCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibSetVertexPositionsCommand::FDNACalibSetVertexPositionsCommand(uint16 MeshIndex, TArrayView<const FVector> Positions, EDNACalibVectorOperation Operation) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndex(MeshIndex);
	ImplPtr->SetPositions(Positions);
	ImplPtr->SetOperation(Operation);
}

FDNACalibSetVertexPositionsCommand::FDNACalibSetVertexPositionsCommand(uint16 MeshIndex, TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs, EDNACalibVectorOperation Operation) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndex(MeshIndex);
	ImplPtr->SetPositions(Xs, Ys, Zs);
	ImplPtr->SetOperation(Operation);
}

FDNACalibSetVertexPositionsCommand::FDNACalibSetVertexPositionsCommand(uint16 MeshIndex, TArrayView<const FVector> Positions, TArrayView<const float> Masks, EDNACalibVectorOperation Operation) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndex(MeshIndex);
	ImplPtr->SetPositions(Positions);
	ImplPtr->SetMasks(Masks);
	ImplPtr->SetOperation(Operation);
}

FDNACalibSetVertexPositionsCommand::FDNACalibSetVertexPositionsCommand(uint16 MeshIndex, TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs, TArrayView<const float> Masks, EDNACalibVectorOperation Operation) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetMeshIndex(MeshIndex);
	ImplPtr->SetPositions(Xs, Ys, Zs);
	ImplPtr->SetMasks(Masks);
	ImplPtr->SetOperation(Operation);
}

FDNACalibSetVertexPositionsCommand::~FDNACalibSetVertexPositionsCommand() = default;
FDNACalibSetVertexPositionsCommand::FDNACalibSetVertexPositionsCommand(FDNACalibSetVertexPositionsCommand&&) = default;
FDNACalibSetVertexPositionsCommand& FDNACalibSetVertexPositionsCommand::operator=(FDNACalibSetVertexPositionsCommand&&) = default;

void FDNACalibSetVertexPositionsCommand::SetMeshIndex(uint16 MeshIndex)
{
	ImplPtr->SetMeshIndex(MeshIndex);
}

void FDNACalibSetVertexPositionsCommand::SetPositions(TArrayView<const FVector> Positions)
{
	ImplPtr->SetPositions(Positions);
}

void FDNACalibSetVertexPositionsCommand::SetPositions(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs)
{
	ImplPtr->SetPositions(Xs, Ys, Zs);
}

void FDNACalibSetVertexPositionsCommand::SetMasks(TArrayView<const float> Masks)
{
	ImplPtr->SetMasks(Masks);
}

void FDNACalibSetVertexPositionsCommand::SetOperation(EDNACalibVectorOperation Operation)
{
	ImplPtr->SetOperation(Operation);
}

void FDNACalibSetVertexPositionsCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
