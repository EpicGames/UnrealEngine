// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DNACalibTranslateCommand.h"

#include "DNACalibDNAReader.h"
#include "DNACalibUtils.h"
#include "FMemoryResource.h"
#include "VecArray.h"

#include "dnacalib/commands/TranslateCommand.h"
#include "dnacalib/dna/DNACalibDNAReader.h"

class FDNACalibTranslateCommand::Impl
{
public:
	Impl() : Command{new dnac::TranslateCommand{FMemoryResource::Instance()}}
	{
	}

	void SetTranslation(FVector Translation)
	{
		Command->setTranslation(dnac::Vector3{static_cast<float>(Translation.X), static_cast<float>(Translation.Y), static_cast<float>(Translation.Z)});
	}

	void Run(FDNACalibDNAReader* Output)
	{
		Command->run(static_cast<dnac::DNACalibDNAReader*>(Output->Unwrap()));
	}

private:
	TUniquePtr<dnac::TranslateCommand> Command;
};

FDNACalibTranslateCommand::FDNACalibTranslateCommand() :
	ImplPtr{new Impl{}}
{
}

FDNACalibTranslateCommand::FDNACalibTranslateCommand(FVector Translation) :
	ImplPtr{new Impl{}}
{
	ImplPtr->SetTranslation(Translation);
}

FDNACalibTranslateCommand::~FDNACalibTranslateCommand() = default;
FDNACalibTranslateCommand::FDNACalibTranslateCommand(FDNACalibTranslateCommand&&) = default;
FDNACalibTranslateCommand& FDNACalibTranslateCommand::operator=(FDNACalibTranslateCommand&&) = default;

void FDNACalibTranslateCommand::SetTranslation(FVector Translation)
{
	ImplPtr->SetTranslation(Translation);
}

void FDNACalibTranslateCommand::Run(FDNACalibDNAReader* Output)
{
	ImplPtr->Run(Output);
}
