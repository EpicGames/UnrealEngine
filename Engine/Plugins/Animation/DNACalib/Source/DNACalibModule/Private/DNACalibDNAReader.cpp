// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNACalibDNAReader.h"

#include "DNAReaderAdapter.h"
#include "FMemoryResource.h"

#include "dnacalib/dna/DNACalibDNAReader.h"

FDNACalibDNAReader::FDNACalibDNAReader(IDNAReader* Source) :
	ReaderPtr{new FDNAReader<dnac::DNACalibDNAReader>{dnac::DNACalibDNAReader::create(Source->Unwrap(), FMemoryResource::Instance()), Source->GetConfig()}}
{
}

dna::Reader* FDNACalibDNAReader::Unwrap() const
{
	return ReaderPtr->Unwrap();
}

/** HEADER READER **/
uint16 FDNACalibDNAReader::GetFileFormatGeneration() const
{
	return ReaderPtr->GetFileFormatGeneration();
}

uint16 FDNACalibDNAReader::GetFileFormatVersion() const
{
	return ReaderPtr->GetFileFormatVersion();
}

/** DESCRIPTOR READER **/

FString FDNACalibDNAReader::GetName() const
{
	return ReaderPtr->GetName();
}

EArchetype FDNACalibDNAReader::GetArchetype() const
{
	return ReaderPtr->GetArchetype();
}

EGender FDNACalibDNAReader::GetGender() const
{
	return ReaderPtr->GetGender();
}

uint16 FDNACalibDNAReader::GetAge() const
{
	return ReaderPtr->GetAge();
}

uint32 FDNACalibDNAReader::GetMetaDataCount() const
{
	return ReaderPtr->GetMetaDataCount();
}

FString FDNACalibDNAReader::GetMetaDataKey(uint32 Index) const
{
	return ReaderPtr->GetMetaDataKey(Index);
}

FString FDNACalibDNAReader::GetMetaDataValue(const FString& Key) const
{
	return ReaderPtr->GetMetaDataValue(Key);
}

ETranslationUnit FDNACalibDNAReader::GetTranslationUnit() const
{
	return ReaderPtr->GetTranslationUnit();
}

ERotationUnit FDNACalibDNAReader::GetRotationUnit() const
{
	return ReaderPtr->GetRotationUnit();
}

FCoordinateSystem FDNACalibDNAReader::GetCoordinateSystem() const
{
	return ReaderPtr->GetCoordinateSystem();
}

ERotationSequence FDNACalibDNAReader::GetRotationSequence() const
{
	return ReaderPtr->GetRotationSequence();
}

FRotationSign FDNACalibDNAReader::GetRotationSign() const
{
	return ReaderPtr->GetRotationSign();
}

EFaceWindingOrder FDNACalibDNAReader::GetFaceWindingOrder() const
{
	return ReaderPtr->GetFaceWindingOrder();
}

uint16 FDNACalibDNAReader::GetLODCount() const
{
	return ReaderPtr->GetLODCount();
}

uint16 FDNACalibDNAReader::GetDBMaxLOD() const
{
	return ReaderPtr->GetDBMaxLOD();
}

FString FDNACalibDNAReader::GetDBComplexity() const
{
	return ReaderPtr->GetDBComplexity();
}

FString FDNACalibDNAReader::GetDBName() const
{
	return ReaderPtr->GetDBName();
}

/** DEFINITION READER **/

uint16 FDNACalibDNAReader::GetGUIControlCount() const
{
	return ReaderPtr->GetGUIControlCount();
}

FString FDNACalibDNAReader::GetGUIControlName(uint16 Index) const
{
	return ReaderPtr->GetGUIControlName(Index);
}

uint16 FDNACalibDNAReader::GetRawControlCount() const
{
	return ReaderPtr->GetRawControlCount();
}

FString FDNACalibDNAReader::GetRawControlName(uint16 Index) const
{
	return ReaderPtr->GetRawControlName(Index);
}

uint16 FDNACalibDNAReader::GetJointCount() const
{
	return ReaderPtr->GetJointCount();
}

FString FDNACalibDNAReader::GetJointName(uint16 Index) const
{
	return ReaderPtr->GetJointName(Index);
}

uint16 FDNACalibDNAReader::GetJointIndexListCount() const
{
	return ReaderPtr->GetJointIndexListCount();
}

TArrayView<const uint16> FDNACalibDNAReader::GetJointIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetJointIndicesForLOD(LOD);
}

uint16 FDNACalibDNAReader::GetBlendShapeChannelCount() const
{
	return ReaderPtr->GetBlendShapeChannelCount();
}

FString FDNACalibDNAReader::GetBlendShapeChannelName(uint16 Index) const
{
	return ReaderPtr->GetBlendShapeChannelName(Index);
}

uint16 FDNACalibDNAReader::GetBlendShapeChannelIndexListCount() const
{
	return ReaderPtr->GetBlendShapeChannelIndexListCount();
}

TArrayView<const uint16> FDNACalibDNAReader::GetBlendShapeChannelIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetBlendShapeChannelIndicesForLOD(LOD);
}

uint16 FDNACalibDNAReader::GetAnimatedMapCount() const
{
	return ReaderPtr->GetAnimatedMapCount();
}

FString FDNACalibDNAReader::GetAnimatedMapName(uint16 Index) const
{
	return ReaderPtr->GetAnimatedMapName(Index);
}

uint16 FDNACalibDNAReader::GetAnimatedMapIndexListCount() const
{
	return ReaderPtr->GetAnimatedMapIndexListCount();
}

TArrayView<const uint16> FDNACalibDNAReader::GetAnimatedMapIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetAnimatedMapIndicesForLOD(LOD);
}

uint16 FDNACalibDNAReader::GetMeshCount() const
{
	return ReaderPtr->GetMeshCount();
}

FString FDNACalibDNAReader::GetMeshName(uint16 Index) const
{
	return ReaderPtr->GetMeshName(Index);
}

uint16 FDNACalibDNAReader::GetMeshIndexListCount() const
{
	return ReaderPtr->GetMeshIndexListCount();
}

TArrayView<const uint16> FDNACalibDNAReader::GetMeshIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetMeshIndicesForLOD(LOD);
}

uint16 FDNACalibDNAReader::GetMeshBlendShapeChannelMappingCount() const
{
	return ReaderPtr->GetMeshBlendShapeChannelMappingCount();
}

FMeshBlendShapeChannelMapping FDNACalibDNAReader::GetMeshBlendShapeChannelMapping(uint16 Index) const
{
	return ReaderPtr->GetMeshBlendShapeChannelMapping(Index);
}

TArrayView<const uint16> FDNACalibDNAReader::GetMeshBlendShapeChannelMappingIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetMeshBlendShapeChannelMappingIndicesForLOD(LOD);
}

FVector FDNACalibDNAReader::GetNeutralJointTranslation(uint16 Index) const
{
	return ReaderPtr->GetNeutralJointTranslation(Index);
}

FVector FDNACalibDNAReader::GetNeutralJointRotation(uint16 Index) const
{
	return ReaderPtr->GetNeutralJointRotation(Index);
}

uint16 FDNACalibDNAReader::GetJointParentIndex(uint16 Index) const
{
	return ReaderPtr->GetJointParentIndex(Index);
}

/** BEHAVIOR READER **/

TArrayView<const uint16> FDNACalibDNAReader::GetGUIToRawInputIndices() const
{
	return ReaderPtr->GetGUIToRawInputIndices();
}

TArrayView<const uint16> FDNACalibDNAReader::GetGUIToRawOutputIndices() const
{
	return ReaderPtr->GetGUIToRawOutputIndices();
}

TArrayView<const float> FDNACalibDNAReader::GetGUIToRawFromValues() const
{
	return ReaderPtr->GetGUIToRawFromValues();
}

TArrayView<const float> FDNACalibDNAReader::GetGUIToRawToValues() const
{
	return  ReaderPtr->GetGUIToRawToValues();
}

TArrayView<const float> FDNACalibDNAReader::GetGUIToRawSlopeValues() const
{
	return ReaderPtr->GetGUIToRawSlopeValues();
}

TArrayView<const float> FDNACalibDNAReader::GetGUIToRawCutValues() const
{
	return ReaderPtr->GetGUIToRawCutValues();
}

uint16 FDNACalibDNAReader::GetPSDCount() const
{
	return ReaderPtr->GetPSDCount();
}

TArrayView<const uint16> FDNACalibDNAReader::GetPSDRowIndices() const
{
	return ReaderPtr->GetPSDRowIndices();
}

TArrayView<const uint16> FDNACalibDNAReader::GetPSDColumnIndices() const
{
	return ReaderPtr->GetPSDColumnIndices();
}

TArrayView<const float> FDNACalibDNAReader::GetPSDValues() const
{
	return ReaderPtr->GetPSDValues();
}

uint16 FDNACalibDNAReader::GetJointRowCount() const
{
	return ReaderPtr->GetJointRowCount();
}

uint16 FDNACalibDNAReader::GetJointColumnCount() const
{
	return ReaderPtr->GetJointColumnCount();
}

TArrayView<const uint16> FDNACalibDNAReader::GetJointGroupJointIndices(uint16 JointGroupIndex) const
{
	return ReaderPtr->GetJointGroupJointIndices(JointGroupIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetJointVariableAttributeIndices(uint16 LOD) const
{
	return ReaderPtr->GetJointVariableAttributeIndices(LOD);
}

uint16 FDNACalibDNAReader::GetJointGroupCount() const
{
	return ReaderPtr->GetJointGroupCount();
}

TArrayView<const uint16> FDNACalibDNAReader::GetJointGroupLODs(uint16 JointGroupIndex) const
{
	return ReaderPtr->GetJointGroupLODs(JointGroupIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetJointGroupInputIndices(uint16 JointGroupIndex) const
{
	return ReaderPtr->GetJointGroupInputIndices(JointGroupIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetJointGroupOutputIndices(uint16 JointGroupIndex) const
{
	return ReaderPtr->GetJointGroupOutputIndices(JointGroupIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetJointGroupValues(uint16 JointGroupIndex) const
{
	return ReaderPtr->GetJointGroupValues(JointGroupIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetBlendShapeChannelLODs() const
{
	return ReaderPtr->GetBlendShapeChannelLODs();
}

TArrayView<const uint16> FDNACalibDNAReader::GetBlendShapeChannelInputIndices() const
{
	return ReaderPtr->GetBlendShapeChannelInputIndices();
}

TArrayView<const uint16> FDNACalibDNAReader::GetBlendShapeChannelOutputIndices() const
{
	return ReaderPtr->GetBlendShapeChannelOutputIndices();
}

TArrayView<const uint16> FDNACalibDNAReader::GetAnimatedMapLODs() const
{
	return ReaderPtr->GetAnimatedMapLODs();
}

TArrayView<const uint16> FDNACalibDNAReader::GetAnimatedMapInputIndices() const
{
	return ReaderPtr->GetAnimatedMapInputIndices();
}

TArrayView<const uint16> FDNACalibDNAReader::GetAnimatedMapOutputIndices() const
{
	return ReaderPtr->GetAnimatedMapOutputIndices();
}

TArrayView<const float> FDNACalibDNAReader::GetAnimatedMapFromValues() const
{
	return ReaderPtr->GetAnimatedMapFromValues();
}

TArrayView<const float> FDNACalibDNAReader::GetAnimatedMapToValues() const
{
	return ReaderPtr->GetAnimatedMapToValues();
}

TArrayView<const float> FDNACalibDNAReader::GetAnimatedMapSlopeValues() const
{
	return ReaderPtr->GetAnimatedMapSlopeValues();
}

TArrayView<const float> FDNACalibDNAReader::GetAnimatedMapCutValues() const
{
	return ReaderPtr->GetAnimatedMapCutValues();
}

/** GEOMETRY READER **/
uint32 FDNACalibDNAReader::GetVertexPositionCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexPositionCount(MeshIndex);
}

FVector FDNACalibDNAReader::GetVertexPosition(uint16 MeshIndex, uint32 PositionIndex) const
{
	return ReaderPtr->GetVertexPosition(MeshIndex, PositionIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetVertexPositionXs(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexPositionXs(MeshIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetVertexPositionYs(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexPositionYs(MeshIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetVertexPositionZs(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexPositionZs(MeshIndex);
}

uint32 FDNACalibDNAReader::GetVertexTextureCoordinateCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexTextureCoordinateCount(MeshIndex);
}

FTextureCoordinate FDNACalibDNAReader::GetVertexTextureCoordinate(uint16 MeshIndex, uint32 TextureCoordinateIndex) const
{
	return ReaderPtr->GetVertexTextureCoordinate(MeshIndex, TextureCoordinateIndex);
}

uint32 FDNACalibDNAReader::GetVertexNormalCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexNormalCount(MeshIndex);
}

FVector FDNACalibDNAReader::GetVertexNormal(uint16 MeshIndex, uint32 NormalIndex) const
{
	return ReaderPtr->GetVertexNormal(MeshIndex, NormalIndex);
}

uint32 FDNACalibDNAReader::GetVertexLayoutCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexLayoutCount(MeshIndex);
}

FVertexLayout FDNACalibDNAReader::GetVertexLayout(uint16 MeshIndex, uint32 VertexIndex) const
{
	return ReaderPtr->GetVertexLayout(MeshIndex, VertexIndex);
}

uint32 FDNACalibDNAReader::GetFaceCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetFaceCount(MeshIndex);
}

TArrayView<const uint32> FDNACalibDNAReader::GetFaceVertexLayoutIndices(uint16 MeshIndex, uint32 FaceIndex) const
{
	return ReaderPtr->GetFaceVertexLayoutIndices(MeshIndex, FaceIndex);
}

uint16 FDNACalibDNAReader::GetMaximumInfluencePerVertex(uint16 MeshIndex) const
{
	return ReaderPtr->GetMaximumInfluencePerVertex(MeshIndex);
}

uint32 FDNACalibDNAReader::GetSkinWeightsCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetSkinWeightsCount(MeshIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetSkinWeightsValues(uint16 MeshIndex, uint32 VertexIndex) const
{
	return ReaderPtr->GetSkinWeightsValues(MeshIndex, VertexIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetSkinWeightsJointIndices(uint16 MeshIndex, uint32 VertexIndex) const
{
	return ReaderPtr->GetSkinWeightsJointIndices(MeshIndex, VertexIndex);
}

uint16 FDNACalibDNAReader::GetBlendShapeTargetCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetBlendShapeTargetCount(MeshIndex);
}

uint16 FDNACalibDNAReader::GetBlendShapeChannelIndex(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	return ReaderPtr->GetBlendShapeChannelIndex(MeshIndex, BlendShapeTargetIndex);
}

uint32 FDNACalibDNAReader::GetBlendShapeTargetDeltaCount(uint16 MeshIndex, uint16 BlendShapeIndex) const
{
	return ReaderPtr->GetBlendShapeTargetDeltaCount(MeshIndex, BlendShapeIndex);
}

FVector FDNACalibDNAReader::GetBlendShapeTargetDelta(uint16 MeshIndex, uint16 BlendShapeIndex, uint32 DeltaIndex) const
{
	return ReaderPtr->GetBlendShapeTargetDelta(MeshIndex, BlendShapeIndex, DeltaIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetBlendShapeTargetDeltaXs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	return ReaderPtr->GetBlendShapeTargetDeltaXs(MeshIndex, BlendShapeTargetIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetBlendShapeTargetDeltaYs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	return ReaderPtr->GetBlendShapeTargetDeltaYs(MeshIndex, BlendShapeTargetIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetBlendShapeTargetDeltaZs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	return ReaderPtr->GetBlendShapeTargetDeltaZs(MeshIndex, BlendShapeTargetIndex);
}

TArrayView<const uint32> FDNACalibDNAReader::GetBlendShapeTargetVertexIndices(uint16 MeshIndex, uint16 BlendShapeIndex) const
{
	return ReaderPtr->GetBlendShapeTargetVertexIndices(MeshIndex, BlendShapeIndex);
}

/** MACHINE LEARNED BEHAVIOR READER **/
uint16 FDNACalibDNAReader::GetMLControlCount() const
{
	return ReaderPtr->GetMLControlCount();
}

FString FDNACalibDNAReader::GetMLControlName(uint16 Index) const
{
	return ReaderPtr->GetMLControlName(Index);
}

uint16 FDNACalibDNAReader::GetNeuralNetworkCount() const
{
	return ReaderPtr->GetNeuralNetworkCount();
}

uint16 FDNACalibDNAReader::GetNeuralNetworkIndexListCount() const
{
	return ReaderPtr->GetNeuralNetworkIndexListCount();
}

TArrayView<const uint16> FDNACalibDNAReader::GetNeuralNetworkIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetNeuralNetworkIndicesForLOD(LOD);
}

uint16 FDNACalibDNAReader::GetMeshRegionCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetMeshRegionCount(MeshIndex);
}

FString FDNACalibDNAReader::GetMeshRegionName(uint16 MeshIndex, uint16 RegionIndex) const
{
	return ReaderPtr->GetMeshRegionName(MeshIndex, RegionIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetNeuralNetworkIndicesForMeshRegion(uint16 MeshIndex, uint16 RegionIndex) const
{
	return ReaderPtr->GetNeuralNetworkIndicesForMeshRegion(MeshIndex, RegionIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetNeuralNetworkInputIndices(uint16 NetIndex) const
{
	return ReaderPtr->GetNeuralNetworkInputIndices(NetIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetNeuralNetworkOutputIndices(uint16 NetIndex) const
{
	return ReaderPtr->GetNeuralNetworkOutputIndices(NetIndex);
}

uint16 FDNACalibDNAReader::GetNeuralNetworkLayerCount(uint16 NetIndex) const
{
	return ReaderPtr->GetNeuralNetworkLayerCount(NetIndex);
}

EActivationFunction FDNACalibDNAReader::GetNeuralNetworkLayerActivationFunction(uint16 NetIndex, uint16 LayerIndex) const
{
	return ReaderPtr->GetNeuralNetworkLayerActivationFunction(NetIndex, LayerIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetNeuralNetworkLayerActivationFunctionParameters(uint16 NetIndex, uint16 LayerIndex) const
{
	return ReaderPtr->GetNeuralNetworkLayerActivationFunctionParameters(NetIndex, LayerIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetNeuralNetworkLayerBiases(uint16 NetIndex, uint16 LayerIndex) const
{
	return ReaderPtr->GetNeuralNetworkLayerBiases(NetIndex, LayerIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetNeuralNetworkLayerWeights(uint16 NetIndex, uint16 LayerIndex) const
{
	return ReaderPtr->GetNeuralNetworkLayerWeights(NetIndex, LayerIndex);
}


ETranslationRepresentation FDNACalibDNAReader::GetJointTranslationRepresentation(uint16 JointIndex) const
{
	return ReaderPtr->GetJointTranslationRepresentation(JointIndex);
}

ERotationRepresentation FDNACalibDNAReader::GetJointRotationRepresentation(uint16 JointIndex) const
{
	return ReaderPtr->GetJointRotationRepresentation(JointIndex);
}

EScaleRepresentation FDNACalibDNAReader::GetJointScaleRepresentation(uint16 JointIndex) const
{
	return ReaderPtr->GetJointScaleRepresentation(JointIndex);
}

uint16 FDNACalibDNAReader::GetRBFPoseCount() const
{
	return ReaderPtr->GetRBFPoseCount();
}

FString FDNACalibDNAReader::GetRBFPoseName(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseName(PoseIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetRBFPoseJointOutputIndices(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseJointOutputIndices(PoseIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetRBFPoseBlendShapeChannelOutputIndices(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseBlendShapeChannelOutputIndices(PoseIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetRBFPoseAnimatedMapOutputIndices(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseAnimatedMapOutputIndices(PoseIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetRBFPoseJointOutputValues(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseJointOutputValues(PoseIndex);
}

float FDNACalibDNAReader::GetRBFPoseScale(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseScale(PoseIndex);
}

uint16 FDNACalibDNAReader::GetRBFPoseControlCount() const
{
	return ReaderPtr->GetRBFPoseControlCount();
}

FString FDNACalibDNAReader::GetRBFPoseControlName(uint16 PoseControlIndex) const
{
	return ReaderPtr->GetRBFPoseControlName(PoseControlIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetRBFPoseInputControlIndices(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseInputControlIndices(PoseIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetRBFPoseOutputControlIndices(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseOutputControlIndices(PoseIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetRBFPoseOutputControlWeights(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseOutputControlWeights(PoseIndex);
}

uint16 FDNACalibDNAReader::GetRBFSolverCount() const
{
	return ReaderPtr->GetRBFSolverCount();
}

uint16 FDNACalibDNAReader::GetRBFSolverIndexListCount() const
{
	return ReaderPtr->GetRBFSolverIndexListCount();
}

TArrayView<const uint16> FDNACalibDNAReader::GetRBFSolverIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetRBFSolverIndicesForLOD(LOD);
}

FString FDNACalibDNAReader::GetRBFSolverName(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverName(SolverIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetRBFSolverRawControlIndices(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverRawControlIndices(SolverIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetRBFSolverPoseIndices(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverPoseIndices(SolverIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetRBFSolverRawControlValues(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverRawControlValues(SolverIndex);
}

ERBFSolverType FDNACalibDNAReader::GetRBFSolverType(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverType(SolverIndex);
}

float FDNACalibDNAReader::GetRBFSolverRadius(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverRadius(SolverIndex);
}

EAutomaticRadius FDNACalibDNAReader::GetRBFSolverAutomaticRadius(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverAutomaticRadius(SolverIndex);
}

float FDNACalibDNAReader::GetRBFSolverWeightThreshold(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverWeightThreshold(SolverIndex);
}

ERBFDistanceMethod FDNACalibDNAReader::GetRBFSolverDistanceMethod(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverDistanceMethod(SolverIndex);
}

ERBFNormalizeMethod FDNACalibDNAReader::GetRBFSolverNormalizeMethod(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverNormalizeMethod(SolverIndex);
}

ERBFFunctionType FDNACalibDNAReader::GetRBFSolverFunctionType(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverFunctionType(SolverIndex);
}

ETwistAxis FDNACalibDNAReader::GetRBFSolverTwistAxis(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverTwistAxis(SolverIndex);
}

uint16 FDNACalibDNAReader::GetTwistCount() const
{
	return ReaderPtr->GetTwistCount();
}

ETwistAxis FDNACalibDNAReader::GetTwistSetupTwistAxis(uint16 TwistIndex) const
{
	return ReaderPtr->GetTwistSetupTwistAxis(TwistIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetTwistInputControlIndices(uint16 TwistIndex) const
{
	return ReaderPtr->GetTwistInputControlIndices(TwistIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetTwistOutputJointIndices(uint16 TwistIndex) const
{
	return ReaderPtr->GetTwistOutputJointIndices(TwistIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetTwistBlendWeights(uint16 TwistIndex) const
{
	return ReaderPtr->GetTwistBlendWeights(TwistIndex);
}

uint16 FDNACalibDNAReader::GetSwingCount() const
{
	return ReaderPtr->GetSwingCount();
}

ETwistAxis FDNACalibDNAReader::GetSwingSetupTwistAxis(uint16 SwingIndex) const
{
	return ReaderPtr->GetSwingSetupTwistAxis(SwingIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetSwingInputControlIndices(uint16 SwingIndex) const
{
	return ReaderPtr->GetSwingInputControlIndices(SwingIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetSwingOutputJointIndices(uint16 SwingIndex) const
{
	return ReaderPtr->GetSwingOutputJointIndices(SwingIndex);
}

TArrayView<const float> FDNACalibDNAReader::GetSwingBlendWeights(uint16 SwingIndex) const
{
	return ReaderPtr->GetSwingBlendWeights(SwingIndex);
}

uint16 FDNACalibDNAReader::GetMLTypeCount() const
{
	return ReaderPtr->GetMLTypeCount();
}

uint16 FDNACalibDNAReader::GetMLOperationSetCount(uint16 MLTypeIndex) const
{
	return ReaderPtr->GetMLOperationSetCount(MLTypeIndex);
}

uint16 FDNACalibDNAReader::GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const
{
	return ReaderPtr->GetMLOperationCount(MLTypeIndex, MLOperationSetIndex);
}

EMachineLearnedBehaviorOperationType FDNACalibDNAReader::GetMLOperationType(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return static_cast<EMachineLearnedBehaviorOperationType>(ReaderPtr->GetMLOperationType(MLTypeIndex, MLOperationSetIndex, MLOperationIndex));
}

TArrayView<const uint32> FDNACalibDNAReader::GetMLOperationParameters(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return ReaderPtr->GetMLOperationParameters(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetMLOperationDependencyOperationSetIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return ReaderPtr->GetMLOperationDependencyOperationSetIndices(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetMLOperationDependencyOperationIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return ReaderPtr->GetMLOperationDependencyOperationIndices(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
}

uint16 FDNACalibDNAReader::GetMLOperationIndexListCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const
{
	return ReaderPtr->GetMLOperationIndexListCount(MLTypeIndex, MLOperationSetIndex);
}

TArrayView<const uint16> FDNACalibDNAReader::GetMLOperationIndicesForLOD(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 LOD) const
{
	return ReaderPtr->GetMLOperationIndicesForLOD(MLTypeIndex, MLOperationSetIndex, LOD);
}

TArrayView<const uint16> FDNACalibDNAReader::GetMLJointsInputIndices() const
{
	return ReaderPtr->GetMLJointsInputIndices();
}

TArrayView<const uint16> FDNACalibDNAReader::GetMLJointsOutputIndices() const
{
	return ReaderPtr->GetMLJointsOutputIndices();
}

TArrayView<const uint16> FDNACalibDNAReader::GetMLJointsParameterKeys() const
{
	return ReaderPtr->GetMLJointsParameterKeys();
}

TArrayView<const uint16> FDNACalibDNAReader::GetMLJointsParameterValues() const
{
	return ReaderPtr->GetMLJointsParameterValues();
}

const FDNAConfig& FDNACalibDNAReader::GetConfig() const
{
	return ReaderPtr->GetConfig();
}

void FDNACalibDNAReader::Unload(EDNADataLayer Layer) {
	return ReaderPtr->Unload(Layer);
}
