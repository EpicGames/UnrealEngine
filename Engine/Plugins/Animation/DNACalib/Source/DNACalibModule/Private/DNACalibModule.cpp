// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNACalibModule.h"

#define LOCTEXT_NAMESPACE "FDNACalibModule"

void FDNACalibModule::StartupModule()
{
}

void FDNACalibModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FDNACalibModule, DNACalibModule)