// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "dnacalib/types/Aliases.h"

template<typename TContainer>
dnac::ConstArrayView<typename TContainer::ElementType> ViewOf(const TContainer& Source)
{
	return dnac::ConstArrayView<typename TContainer::ElementType>{Source.GetData(), static_cast<size_t>(Source.Num())};
}
