// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "Math/Vector.h"

struct FVecArray
{
	TArray<float> Xs;
	TArray<float> Ys;
	TArray<float> Zs;

	FVecArray() = default;

	template<typename TVec>
	FVecArray(TArrayView<TVec> Source)
	{
		Assign(Source);
	}

	template<typename T>
	FVecArray(TArrayView<T> SourceXs, TArrayView<T> SourceYs, TArrayView<T> SourceZs)
	{
		Assign(SourceXs, SourceYs, SourceZs);
	}

	int Num()
	{
		check((Xs.Num() == Ys.Num()) && (Ys.Num() == Zs.Num()));
		return Xs.Num();
	}

	void Empty()
	{
		Xs.Empty();
		Ys.Empty();
		Zs.Empty();
	}

	void Reserve(int Number)
	{
		Xs.Reserve(Number);
		Ys.Reserve(Number);
		Zs.Reserve(Number);
	}

	void AddUninitialized(int Count)
	{
		Xs.AddUninitialized(Count);
		Ys.AddUninitialized(Count);
		Zs.AddUninitialized(Count);
	}

	template<typename TVec>
	void Assign(TArrayView<TVec> Source)
	{
		Empty();
		Reserve(Source.Num());
		for (const auto& Vec : Source)
		{
			Xs.Add(Vec.X);
			Ys.Add(Vec.Y);
			Zs.Add(Vec.Z);
		}
	}

	template<typename T>
	void Assign(TArrayView<T> SourceXs, TArrayView<T> SourceYs, TArrayView<T> SourceZs)
	{
		check((SourceXs.Num() == SourceYs.Num()) && (SourceYs.Num() == SourceZs.Num()));
		Empty();
		AddUninitialized(SourceXs.Num());
		FMemory::Memcpy(Xs.GetData(), SourceXs.GetData(), SourceXs.Num() * sizeof(float));
		FMemory::Memcpy(Ys.GetData(), SourceYs.GetData(), SourceYs.Num() * sizeof(float));
		FMemory::Memcpy(Zs.GetData(), SourceZs.GetData(), SourceZs.Num() * sizeof(float));
	}
};
