// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibCalculateMeshLowerLODsCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibCalculateMeshLowerLODsCommand();
	DNACALIBMODULE_API FDNACalibCalculateMeshLowerLODsCommand(uint16 MeshIndex);

	DNACALIBMODULE_API ~FDNACalibCalculateMeshLowerLODsCommand();

	FDNACalibCalculateMeshLowerLODsCommand(const FDNACalibCalculateMeshLowerLODsCommand&) = delete;
	FDNACalibCalculateMeshLowerLODsCommand& operator=(const FDNACalibCalculateMeshLowerLODsCommand&) = delete;

	DNACALIBMODULE_API FDNACalibCalculateMeshLowerLODsCommand(FDNACalibCalculateMeshLowerLODsCommand&&);
	DNACALIBMODULE_API FDNACalibCalculateMeshLowerLODsCommand& operator=(FDNACalibCalculateMeshLowerLODsCommand&&);

	DNACALIBMODULE_API void SetMeshIndex(uint16 MeshIndex);
	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
