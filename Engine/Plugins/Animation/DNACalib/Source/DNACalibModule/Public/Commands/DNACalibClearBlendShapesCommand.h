// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibClearBlendShapesCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibClearBlendShapesCommand();

	DNACALIBMODULE_API ~FDNACalibClearBlendShapesCommand();

	FDNACalibClearBlendShapesCommand(const FDNACalibClearBlendShapesCommand&) = delete;
	FDNACalibClearBlendShapesCommand& operator=(const FDNACalibClearBlendShapesCommand&) = delete;

	DNACALIBMODULE_API FDNACalibClearBlendShapesCommand(FDNACalibClearBlendShapesCommand&&);
	DNACALIBMODULE_API FDNACalibClearBlendShapesCommand& operator=(FDNACalibClearBlendShapesCommand&&);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
