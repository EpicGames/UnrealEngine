// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibCommandSequence : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibCommandSequence();
	DNACALIBMODULE_API ~FDNACalibCommandSequence();

	FDNACalibCommandSequence(const FDNACalibCommandSequence&) = delete;
	FDNACalibCommandSequence& operator=(const FDNACalibCommandSequence&) = delete;

	DNACALIBMODULE_API FDNACalibCommandSequence(FDNACalibCommandSequence&&);
	DNACALIBMODULE_API FDNACalibCommandSequence& operator=(FDNACalibCommandSequence&&);

	DNACALIBMODULE_API void Add(IDNACalibCommand* Command);
	DNACALIBMODULE_API void Add(TArrayView<IDNACalibCommand> Commands);

	template<class ... TCommands>
	void Add(TCommands... Commands) {
		static_assert(sizeof...(TCommands) > 0, "At least one command must be passed.");
		IDNACalibCommand* CommandList[] = {Commands ...};
		for (auto Cmd : CommandList) {
			Add(Cmd);
		}
	}

	DNACALIBMODULE_API void Remove(IDNACalibCommand* Command);
	DNACALIBMODULE_API void Remove(TArrayView<IDNACalibCommand> Commands);

	template<class ... TCommands>
	void Remove(TCommands... Commands) {
		static_assert(sizeof...(Commands) > 0, "At least one command must be passed.");
		IDNACalibCommand* CommandList[] = {Commands ...};
		for (auto Cmd : CommandList) {
			Remove(Cmd);
		}
	}

	DNACALIBMODULE_API bool Contains(IDNACalibCommand* Command) const;
	DNACALIBMODULE_API size_t Size() const;

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
