// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class IDNAReader;

class FDNACalibComputeVertexPositionDeltasCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibComputeVertexPositionDeltasCommand();
	DNACALIBMODULE_API FDNACalibComputeVertexPositionDeltasCommand(IDNAReader* ReaderA, IDNAReader* ReaderB);

	DNACALIBMODULE_API ~FDNACalibComputeVertexPositionDeltasCommand();

	FDNACalibComputeVertexPositionDeltasCommand(const FDNACalibComputeVertexPositionDeltasCommand&) = delete;
	FDNACalibComputeVertexPositionDeltasCommand& operator=(const FDNACalibComputeVertexPositionDeltasCommand&) = delete;

	DNACALIBMODULE_API FDNACalibComputeVertexPositionDeltasCommand(FDNACalibComputeVertexPositionDeltasCommand&&);
	DNACALIBMODULE_API FDNACalibComputeVertexPositionDeltasCommand& operator=(FDNACalibComputeVertexPositionDeltasCommand&&);

	DNACALIBMODULE_API void SetReaderA(IDNAReader* Reader);
	DNACALIBMODULE_API void SetReaderB(IDNAReader* Reader);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
