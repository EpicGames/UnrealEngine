// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibConditionalCommand : public IDNACalibCommand {
public:
	using TCondition = TFunction<bool(IDNACalibCommand*, FDNACalibDNAReader*)>;

public:
	DNACALIBMODULE_API FDNACalibConditionalCommand();
	DNACALIBMODULE_API FDNACalibConditionalCommand(IDNACalibCommand* Command, TCondition Condition);

	DNACALIBMODULE_API ~FDNACalibConditionalCommand();

	FDNACalibConditionalCommand(const FDNACalibConditionalCommand&) = delete;
	FDNACalibConditionalCommand& operator=(const FDNACalibConditionalCommand&) = delete;

	DNACALIBMODULE_API FDNACalibConditionalCommand(FDNACalibConditionalCommand&&);
	DNACALIBMODULE_API FDNACalibConditionalCommand& operator=(FDNACalibConditionalCommand&&);

	DNACALIBMODULE_API void SetCommand(IDNACalibCommand* Command);
	DNACALIBMODULE_API void SetCondition(TCondition Condition);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
