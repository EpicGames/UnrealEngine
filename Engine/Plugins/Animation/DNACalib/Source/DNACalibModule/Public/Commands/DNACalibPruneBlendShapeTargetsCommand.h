// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibPruneBlendShapeTargetsCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibPruneBlendShapeTargetsCommand();
	DNACALIBMODULE_API FDNACalibPruneBlendShapeTargetsCommand(float Threshold);

	DNACALIBMODULE_API ~FDNACalibPruneBlendShapeTargetsCommand();

	FDNACalibPruneBlendShapeTargetsCommand(const FDNACalibPruneBlendShapeTargetsCommand&) = delete;
	FDNACalibPruneBlendShapeTargetsCommand& operator=(const FDNACalibPruneBlendShapeTargetsCommand&) = delete;

	DNACALIBMODULE_API FDNACalibPruneBlendShapeTargetsCommand(FDNACalibPruneBlendShapeTargetsCommand&&);
	DNACALIBMODULE_API FDNACalibPruneBlendShapeTargetsCommand& operator=(FDNACalibPruneBlendShapeTargetsCommand&&);

	DNACALIBMODULE_API void SetThreshold(float Threshold);
	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
