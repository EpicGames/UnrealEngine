// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibRemoveAnimatedMapCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibRemoveAnimatedMapCommand();
	DNACALIBMODULE_API FDNACalibRemoveAnimatedMapCommand(uint16 AnimatedMapIndex);
	DNACALIBMODULE_API FDNACalibRemoveAnimatedMapCommand(TArrayView<uint16> AnimatedMapIndices);

	DNACALIBMODULE_API ~FDNACalibRemoveAnimatedMapCommand();

	FDNACalibRemoveAnimatedMapCommand(const FDNACalibRemoveAnimatedMapCommand&) = delete;
	FDNACalibRemoveAnimatedMapCommand& operator=(const FDNACalibRemoveAnimatedMapCommand&) = delete;

	DNACALIBMODULE_API FDNACalibRemoveAnimatedMapCommand(FDNACalibRemoveAnimatedMapCommand&&);
	DNACALIBMODULE_API FDNACalibRemoveAnimatedMapCommand& operator=(FDNACalibRemoveAnimatedMapCommand&&);

	DNACALIBMODULE_API void SetAnimatedMapIndex(uint16 AnimatedMapIndex);
	DNACALIBMODULE_API void SetAnimatedMapIndices(TArrayView<uint16> AnimatedMapIndices);
	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
