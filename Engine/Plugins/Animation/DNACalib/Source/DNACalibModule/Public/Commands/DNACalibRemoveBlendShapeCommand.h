// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibRemoveBlendShapeCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibRemoveBlendShapeCommand();
	DNACALIBMODULE_API FDNACalibRemoveBlendShapeCommand(uint16 BlendShapeIndex);
	DNACALIBMODULE_API FDNACalibRemoveBlendShapeCommand(TArrayView<uint16> BlendShapeIndices);

	DNACALIBMODULE_API ~FDNACalibRemoveBlendShapeCommand();

	FDNACalibRemoveBlendShapeCommand(const FDNACalibRemoveBlendShapeCommand&) = delete;
	FDNACalibRemoveBlendShapeCommand& operator=(const FDNACalibRemoveBlendShapeCommand&) = delete;

	DNACALIBMODULE_API FDNACalibRemoveBlendShapeCommand(FDNACalibRemoveBlendShapeCommand&&);
	DNACALIBMODULE_API FDNACalibRemoveBlendShapeCommand& operator=(FDNACalibRemoveBlendShapeCommand&&);

	DNACALIBMODULE_API void SetBlendShapeIndex(uint16 BlendShapeIndex);
	DNACALIBMODULE_API void SetBlendShapeIndices(TArrayView<uint16> BlendShapeIndices);
	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
