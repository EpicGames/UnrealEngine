// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibRemoveJointAnimationCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibRemoveJointAnimationCommand();
	DNACALIBMODULE_API FDNACalibRemoveJointAnimationCommand(uint16 JointIndex);
	DNACALIBMODULE_API FDNACalibRemoveJointAnimationCommand(TArrayView<uint16> JointIndices);

	DNACALIBMODULE_API ~FDNACalibRemoveJointAnimationCommand();

	FDNACalibRemoveJointAnimationCommand(const FDNACalibRemoveJointAnimationCommand&) = delete;
	FDNACalibRemoveJointAnimationCommand& operator=(const FDNACalibRemoveJointAnimationCommand&) = delete;

	DNACALIBMODULE_API FDNACalibRemoveJointAnimationCommand(FDNACalibRemoveJointAnimationCommand&&);
	DNACALIBMODULE_API FDNACalibRemoveJointAnimationCommand& operator=(FDNACalibRemoveJointAnimationCommand&&);

	DNACALIBMODULE_API void SetJointIndex(uint16 JointIndex);
	DNACALIBMODULE_API void SetJointIndices(TArrayView<uint16> JointIndices);
	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
