// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibRemoveJointCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibRemoveJointCommand();
	DNACALIBMODULE_API FDNACalibRemoveJointCommand(uint16 JointIndex);
	DNACALIBMODULE_API FDNACalibRemoveJointCommand(TArrayView<uint16> JointIndices);

	DNACALIBMODULE_API ~FDNACalibRemoveJointCommand();

	FDNACalibRemoveJointCommand(const FDNACalibRemoveJointCommand&) = delete;
	FDNACalibRemoveJointCommand& operator=(const FDNACalibRemoveJointCommand&) = delete;

	DNACALIBMODULE_API FDNACalibRemoveJointCommand(FDNACalibRemoveJointCommand&&);
	DNACALIBMODULE_API FDNACalibRemoveJointCommand& operator=(FDNACalibRemoveJointCommand&&);

	DNACALIBMODULE_API void SetJointIndex(uint16 JointIndex);
	DNACALIBMODULE_API void SetJointIndices(TArrayView<uint16> JointIndices);
	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
