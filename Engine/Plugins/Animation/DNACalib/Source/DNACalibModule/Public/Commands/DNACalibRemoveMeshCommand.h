// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibRemoveMeshCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibRemoveMeshCommand();
	DNACALIBMODULE_API FDNACalibRemoveMeshCommand(uint16 MeshIndex);
	DNACALIBMODULE_API FDNACalibRemoveMeshCommand(TArrayView<uint16> MeshIndices);

	DNACALIBMODULE_API ~FDNACalibRemoveMeshCommand();

	FDNACalibRemoveMeshCommand(const FDNACalibRemoveMeshCommand&) = delete;
	FDNACalibRemoveMeshCommand& operator=(const FDNACalibRemoveMeshCommand&) = delete;

	DNACALIBMODULE_API FDNACalibRemoveMeshCommand(FDNACalibRemoveMeshCommand&&);
	DNACALIBMODULE_API FDNACalibRemoveMeshCommand& operator=(FDNACalibRemoveMeshCommand&&);

	DNACALIBMODULE_API void SetMeshIndex(uint16 MeshIndex);
	DNACALIBMODULE_API void SetMeshIndices(TArrayView<uint16> MeshIndices);
	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
