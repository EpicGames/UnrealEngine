// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibRenameAnimatedMapCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibRenameAnimatedMapCommand();
	DNACALIBMODULE_API FDNACalibRenameAnimatedMapCommand(uint16 AnimatedMapIndex, const FString& NewName);
	DNACALIBMODULE_API FDNACalibRenameAnimatedMapCommand(const FString& OldName, const FString& NewName);

	DNACALIBMODULE_API ~FDNACalibRenameAnimatedMapCommand();

	FDNACalibRenameAnimatedMapCommand(const FDNACalibRenameAnimatedMapCommand&) = delete;
	FDNACalibRenameAnimatedMapCommand& operator=(const FDNACalibRenameAnimatedMapCommand&) = delete;

	DNACALIBMODULE_API FDNACalibRenameAnimatedMapCommand(FDNACalibRenameAnimatedMapCommand&&);
	DNACALIBMODULE_API FDNACalibRenameAnimatedMapCommand& operator=(FDNACalibRenameAnimatedMapCommand&&);

	DNACALIBMODULE_API void SetName(uint16 AnimatedMapIndex, const FString& NewName);
	DNACALIBMODULE_API void SetName(const FString& OldName, const FString& NewName);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
