// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibRenameBlendShapeCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibRenameBlendShapeCommand();
	DNACALIBMODULE_API FDNACalibRenameBlendShapeCommand(uint16 BlendShapeIndex, const FString& NewName);
	DNACALIBMODULE_API FDNACalibRenameBlendShapeCommand(const FString& OldName, const FString& NewName);

	DNACALIBMODULE_API ~FDNACalibRenameBlendShapeCommand();

	FDNACalibRenameBlendShapeCommand(const FDNACalibRenameBlendShapeCommand&) = delete;
	FDNACalibRenameBlendShapeCommand& operator=(const FDNACalibRenameBlendShapeCommand&) = delete;

	DNACALIBMODULE_API FDNACalibRenameBlendShapeCommand(FDNACalibRenameBlendShapeCommand&&);
	DNACALIBMODULE_API FDNACalibRenameBlendShapeCommand& operator=(FDNACalibRenameBlendShapeCommand&&);

	DNACALIBMODULE_API void SetName(uint16 BlendShapeIndex, const FString& NewName);
	DNACALIBMODULE_API void SetName(const FString& OldName, const FString& NewName);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
