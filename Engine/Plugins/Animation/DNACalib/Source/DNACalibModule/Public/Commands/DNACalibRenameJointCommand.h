// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibRenameJointCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibRenameJointCommand();
	DNACALIBMODULE_API FDNACalibRenameJointCommand(uint16 JointIndex, const FString& NewName);
	DNACALIBMODULE_API FDNACalibRenameJointCommand(const FString& OldName, const FString& NewName);

	DNACALIBMODULE_API ~FDNACalibRenameJointCommand();

	FDNACalibRenameJointCommand(const FDNACalibRenameJointCommand&) = delete;
	FDNACalibRenameJointCommand& operator=(const FDNACalibRenameJointCommand&) = delete;

	DNACALIBMODULE_API FDNACalibRenameJointCommand(FDNACalibRenameJointCommand&&);
	DNACALIBMODULE_API FDNACalibRenameJointCommand& operator=(FDNACalibRenameJointCommand&&);

	DNACALIBMODULE_API void SetName(uint16 JointIndex, const FString& NewName);
	DNACALIBMODULE_API void SetName(const FString& OldName, const FString& NewName);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
