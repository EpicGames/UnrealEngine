// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibRenameMeshCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibRenameMeshCommand();
	DNACALIBMODULE_API FDNACalibRenameMeshCommand(uint16 MeshIndex, const FString& NewName);
	DNACALIBMODULE_API FDNACalibRenameMeshCommand(const FString& OldName, const FString& NewName);

	DNACALIBMODULE_API ~FDNACalibRenameMeshCommand();

	FDNACalibRenameMeshCommand(const FDNACalibRenameMeshCommand&) = delete;
	FDNACalibRenameMeshCommand& operator=(const FDNACalibRenameMeshCommand&) = delete;

	DNACALIBMODULE_API FDNACalibRenameMeshCommand(FDNACalibRenameMeshCommand&&);
	DNACALIBMODULE_API FDNACalibRenameMeshCommand& operator=(FDNACalibRenameMeshCommand&&);

	DNACALIBMODULE_API void SetName(uint16 MeshIndex, const FString& NewName);
	DNACALIBMODULE_API void SetName(const FString& OldName, const FString& NewName);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
