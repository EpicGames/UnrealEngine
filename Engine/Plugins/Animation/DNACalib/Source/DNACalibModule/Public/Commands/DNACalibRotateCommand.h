// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibRotateCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibRotateCommand();
	DNACALIBMODULE_API FDNACalibRotateCommand(FVector Degrees, FVector Origin);

	DNACALIBMODULE_API ~FDNACalibRotateCommand();

	FDNACalibRotateCommand(const FDNACalibRotateCommand&) = delete;
	FDNACalibRotateCommand& operator=(const FDNACalibRotateCommand&) = delete;

	DNACALIBMODULE_API FDNACalibRotateCommand(FDNACalibRotateCommand&&);
	DNACALIBMODULE_API FDNACalibRotateCommand& operator=(FDNACalibRotateCommand&&);

	DNACALIBMODULE_API void SetRotation(FVector Degrees);
	DNACALIBMODULE_API void SetOrigin(FVector Origin);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
