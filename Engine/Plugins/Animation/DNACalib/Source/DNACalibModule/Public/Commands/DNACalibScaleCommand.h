// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibScaleCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibScaleCommand();
	DNACALIBMODULE_API FDNACalibScaleCommand(float Scale, FVector Origin);

	DNACALIBMODULE_API ~FDNACalibScaleCommand();

	FDNACalibScaleCommand(const FDNACalibScaleCommand&) = delete;
	FDNACalibScaleCommand& operator=(const FDNACalibScaleCommand&) = delete;

	DNACALIBMODULE_API FDNACalibScaleCommand(FDNACalibScaleCommand&&);
	DNACALIBMODULE_API FDNACalibScaleCommand& operator=(FDNACalibScaleCommand&&);

	DNACALIBMODULE_API void SetScale(float Scale);
	DNACALIBMODULE_API void SetOrigin(FVector Origin);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
