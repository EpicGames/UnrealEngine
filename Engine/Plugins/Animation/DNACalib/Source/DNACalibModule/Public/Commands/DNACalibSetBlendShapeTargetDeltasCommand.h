// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"
#include "DNACalibVectorOperation.h"

class FDNACalibSetBlendShapeTargetDeltasCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibSetBlendShapeTargetDeltasCommand();
	DNACALIBMODULE_API FDNACalibSetBlendShapeTargetDeltasCommand(uint16 MeshIndex, uint16 BlendShapeTargetIndex, TArrayView<const FVector> Deltas, TArrayView<const uint32> VertexIndices, EDNACalibVectorOperation Operation);
	DNACALIBMODULE_API FDNACalibSetBlendShapeTargetDeltasCommand(uint16 MeshIndex, uint16 BlendShapeTargetIndex, TArrayView<const float> DXs, TArrayView<const float> DYs, TArrayView<const float> DZs, TArrayView<const uint32> VertexIndices, EDNACalibVectorOperation Operation);
	DNACALIBMODULE_API FDNACalibSetBlendShapeTargetDeltasCommand(uint16 MeshIndex, uint16 BlendShapeTargetIndex, TArrayView<const FVector> Deltas, TArrayView<const uint32> VertexIndices, TArrayView<const float> Masks, EDNACalibVectorOperation Operation);
	DNACALIBMODULE_API FDNACalibSetBlendShapeTargetDeltasCommand(uint16 MeshIndex, uint16 BlendShapeTargetIndex, TArrayView<const float> DXs, TArrayView<const float> DYs, TArrayView<const float> DZs, TArrayView<const uint32> VertexIndices, TArrayView<const float> Masks, EDNACalibVectorOperation Operation);

	DNACALIBMODULE_API ~FDNACalibSetBlendShapeTargetDeltasCommand();

	FDNACalibSetBlendShapeTargetDeltasCommand(const FDNACalibSetBlendShapeTargetDeltasCommand&) = delete;
	FDNACalibSetBlendShapeTargetDeltasCommand& operator=(const FDNACalibSetBlendShapeTargetDeltasCommand&) = delete;

	DNACALIBMODULE_API FDNACalibSetBlendShapeTargetDeltasCommand(FDNACalibSetBlendShapeTargetDeltasCommand&&);
	DNACALIBMODULE_API FDNACalibSetBlendShapeTargetDeltasCommand& operator=(FDNACalibSetBlendShapeTargetDeltasCommand&&);

	DNACALIBMODULE_API void SetMeshIndex(uint16 MeshIndex);
	DNACALIBMODULE_API void SetBlendShapeTargetIndex(uint16 BlendShapeTargetIndex);
	DNACALIBMODULE_API void SetDeltas(TArrayView<const FVector> Deltas);
	DNACALIBMODULE_API void SetDeltas(TArrayView<const float> DXs, TArrayView<const float> DYs, TArrayView<const float> DZs);
	DNACALIBMODULE_API void SetVertexIndices(TArrayView<const uint32> VertexIndices);
	DNACALIBMODULE_API void SetMasks(TArrayView<const float> Masks);
	DNACALIBMODULE_API void SetOperation(EDNACalibVectorOperation Operation);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
