// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibSetLODsCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibSetLODsCommand();
	DNACALIBMODULE_API FDNACalibSetLODsCommand(TArrayView<const uint16> LODs);

	DNACALIBMODULE_API ~FDNACalibSetLODsCommand();

	FDNACalibSetLODsCommand(const FDNACalibSetLODsCommand&) = delete;
	FDNACalibSetLODsCommand& operator=(const FDNACalibSetLODsCommand&) = delete;

	DNACALIBMODULE_API FDNACalibSetLODsCommand(FDNACalibSetLODsCommand&&);
	DNACALIBMODULE_API FDNACalibSetLODsCommand& operator=(FDNACalibSetLODsCommand&&);

	DNACALIBMODULE_API void SetLODs(TArrayView<const uint16> LODs);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
