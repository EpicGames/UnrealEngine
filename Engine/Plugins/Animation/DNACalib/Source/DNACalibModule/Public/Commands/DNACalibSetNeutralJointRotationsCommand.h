// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibSetNeutralJointRotationsCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibSetNeutralJointRotationsCommand();
	DNACALIBMODULE_API FDNACalibSetNeutralJointRotationsCommand(TArrayView<const FVector> Positions);
	DNACALIBMODULE_API FDNACalibSetNeutralJointRotationsCommand(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs);

	DNACALIBMODULE_API ~FDNACalibSetNeutralJointRotationsCommand();

	FDNACalibSetNeutralJointRotationsCommand(const FDNACalibSetNeutralJointRotationsCommand&) = delete;
	FDNACalibSetNeutralJointRotationsCommand& operator=(const FDNACalibSetNeutralJointRotationsCommand&) = delete;

	DNACALIBMODULE_API FDNACalibSetNeutralJointRotationsCommand(FDNACalibSetNeutralJointRotationsCommand&&);
	DNACALIBMODULE_API FDNACalibSetNeutralJointRotationsCommand& operator=(FDNACalibSetNeutralJointRotationsCommand&&);

	DNACALIBMODULE_API void SetRotations(TArrayView<const FVector> Positions);
	DNACALIBMODULE_API void SetRotations(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
