// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibSetNeutralJointTranslationsCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibSetNeutralJointTranslationsCommand();
	DNACALIBMODULE_API FDNACalibSetNeutralJointTranslationsCommand(TArrayView<const FVector> Translations);
	DNACALIBMODULE_API FDNACalibSetNeutralJointTranslationsCommand(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs);

	DNACALIBMODULE_API ~FDNACalibSetNeutralJointTranslationsCommand();

	FDNACalibSetNeutralJointTranslationsCommand(const FDNACalibSetNeutralJointTranslationsCommand&) = delete;
	FDNACalibSetNeutralJointTranslationsCommand& operator=(const FDNACalibSetNeutralJointTranslationsCommand&) = delete;

	DNACALIBMODULE_API FDNACalibSetNeutralJointTranslationsCommand(FDNACalibSetNeutralJointTranslationsCommand&&);
	DNACALIBMODULE_API FDNACalibSetNeutralJointTranslationsCommand& operator=(FDNACalibSetNeutralJointTranslationsCommand&&);

	DNACALIBMODULE_API void SetTranslations(TArrayView<const FVector> Translations);
	DNACALIBMODULE_API void SetTranslations(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
