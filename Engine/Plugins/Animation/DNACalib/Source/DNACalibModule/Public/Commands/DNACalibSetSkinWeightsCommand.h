// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibSetSkinWeightsCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibSetSkinWeightsCommand();
	DNACALIBMODULE_API FDNACalibSetSkinWeightsCommand(uint16 MeshIndex, uint32 VertexIndex, TArrayView<const float> Weights, TArrayView<const uint16> JointIndices);

	DNACALIBMODULE_API ~FDNACalibSetSkinWeightsCommand();

	FDNACalibSetSkinWeightsCommand(const FDNACalibSetSkinWeightsCommand&) = delete;
	FDNACalibSetSkinWeightsCommand& operator=(const FDNACalibSetSkinWeightsCommand&) = delete;

	DNACALIBMODULE_API FDNACalibSetSkinWeightsCommand(FDNACalibSetSkinWeightsCommand&&);
	DNACALIBMODULE_API FDNACalibSetSkinWeightsCommand& operator=(FDNACalibSetSkinWeightsCommand&&);

	DNACALIBMODULE_API void SetMeshIndex(uint16 MeshIndex);
	DNACALIBMODULE_API void SetVertexIndex(uint32 VertexIndex);
	DNACALIBMODULE_API void SetWeights(TArrayView<const float> Weights);
	DNACALIBMODULE_API void SetJointIndices(TArrayView<const uint16> JointIndices);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
