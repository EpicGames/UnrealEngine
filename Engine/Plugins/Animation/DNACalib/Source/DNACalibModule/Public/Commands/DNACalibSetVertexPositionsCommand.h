// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"
#include "DNACalibVectorOperation.h"

class FDNACalibSetVertexPositionsCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibSetVertexPositionsCommand();
	DNACALIBMODULE_API FDNACalibSetVertexPositionsCommand(uint16 MeshIndex, TArrayView<const FVector> Positions, EDNACalibVectorOperation Operation);
	DNACALIBMODULE_API FDNACalibSetVertexPositionsCommand(uint16 MeshIndex, TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs, EDNACalibVectorOperation Operation);
	DNACALIBMODULE_API FDNACalibSetVertexPositionsCommand(uint16 MeshIndex, TArrayView<const FVector> Positions, TArrayView<const float> Masks, EDNACalibVectorOperation Operation);
	DNACALIBMODULE_API FDNACalibSetVertexPositionsCommand(uint16 MeshIndex, TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs, TArrayView<const float> Masks, EDNACalibVectorOperation Operation);

	DNACALIBMODULE_API ~FDNACalibSetVertexPositionsCommand();

	FDNACalibSetVertexPositionsCommand(const FDNACalibSetVertexPositionsCommand&) = delete;
	FDNACalibSetVertexPositionsCommand& operator=(const FDNACalibSetVertexPositionsCommand&) = delete;

	DNACALIBMODULE_API FDNACalibSetVertexPositionsCommand(FDNACalibSetVertexPositionsCommand&&);
	DNACALIBMODULE_API FDNACalibSetVertexPositionsCommand& operator=(FDNACalibSetVertexPositionsCommand&&);

	DNACALIBMODULE_API void SetMeshIndex(uint16 MeshIndex);
	DNACALIBMODULE_API void SetPositions(TArrayView<const FVector> Positions);
	DNACALIBMODULE_API void SetPositions(TArrayView<const float> Xs, TArrayView<const float> Ys, TArrayView<const float> Zs);
	DNACALIBMODULE_API void SetMasks(TArrayView<const float> Masks);
	DNACALIBMODULE_API void SetOperation(EDNACalibVectorOperation Operation);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
