// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibCommand.h"

class FDNACalibTranslateCommand : public IDNACalibCommand {
public:
	DNACALIBMODULE_API FDNACalibTranslateCommand();
	DNACALIBMODULE_API FDNACalibTranslateCommand(FVector Translation);

	DNACALIBMODULE_API ~FDNACalibTranslateCommand();

	FDNACalibTranslateCommand(const FDNACalibTranslateCommand&) = delete;
	FDNACalibTranslateCommand& operator=(const FDNACalibTranslateCommand&) = delete;

	DNACALIBMODULE_API FDNACalibTranslateCommand(FDNACalibTranslateCommand&&);
	DNACALIBMODULE_API FDNACalibTranslateCommand& operator=(FDNACalibTranslateCommand&&);

	DNACALIBMODULE_API void SetTranslation(FVector Translation);

	DNACALIBMODULE_API void Run(FDNACalibDNAReader* Output) override;

private:
	class Impl;
	TUniquePtr<Impl> ImplPtr;
};
