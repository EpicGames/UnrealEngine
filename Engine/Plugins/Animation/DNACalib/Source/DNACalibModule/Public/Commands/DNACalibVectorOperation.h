// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACalibVectorOperation.generated.h"

UENUM(BlueprintType)
enum class EDNACalibVectorOperation: uint8
{
	Interpolate,
	Add,
	Subtract,
	Multiply
};