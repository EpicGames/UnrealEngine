// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FDNACalibDNAReader;

/**
	@brief UE Interface for DNACalib Command wrappers.
*/
class IDNACalibCommand
{
public:
	virtual ~IDNACalibCommand() = default;
	virtual void Run(FDNACalibDNAReader* Output) = 0;

};
