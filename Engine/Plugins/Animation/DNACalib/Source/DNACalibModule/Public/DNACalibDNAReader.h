// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DNAReader.h"

#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#define UE_API DNACALIBMODULE_API

/**
	@brief A UE wrapper for a special purpose DNA Reader type which serves as the output parameter of DNACalib.
*/
class FDNACalibDNAReader : public IDNAReader
{
public:
	UE_API explicit FDNACalibDNAReader(IDNAReader* Source);

	// Header
	UE_API uint16 GetFileFormatGeneration() const override;
	UE_API uint16 GetFileFormatVersion() const override;
	// Descriptor
	UE_API FString GetName() const override;
	UE_API EArchetype GetArchetype() const override;
	UE_API EGender GetGender() const override;
	UE_API uint16 GetAge() const override;
	UE_API uint32 GetMetaDataCount() const override;
	UE_API FString GetMetaDataKey(uint32 Index) const override;
	UE_API FString GetMetaDataValue(const FString& Key) const override;
	UE_API ETranslationUnit GetTranslationUnit() const override;
	UE_API ERotationUnit GetRotationUnit() const override;
	UE_API FCoordinateSystem GetCoordinateSystem() const override;
	UE_API ERotationSequence GetRotationSequence() const override;
	UE_API FRotationSign GetRotationSign() const override;
	UE_API EFaceWindingOrder GetFaceWindingOrder() const override;
	UE_API uint16 GetLODCount() const override;
	UE_API uint16 GetDBMaxLOD() const override;
	UE_API FString GetDBComplexity() const override;
	UE_API FString GetDBName() const override;
	// Definition
	UE_API uint16 GetGUIControlCount() const override;
	UE_API FString GetGUIControlName(uint16 Index) const override;
	UE_API uint16 GetRawControlCount() const override;
	UE_API FString GetRawControlName(uint16 Index) const override;
	UE_API uint16 GetJointCount() const override;
	UE_API FString GetJointName(uint16 Index) const override;
	UE_API uint16 GetJointIndexListCount() const override;
	UE_API TArrayView<const uint16> GetJointIndicesForLOD(uint16 LOD) const override;
	UE_API uint16 GetJointParentIndex(uint16 Index) const override;
	UE_API uint16 GetBlendShapeChannelCount() const override;
	UE_API FString GetBlendShapeChannelName(uint16 Index) const override;
	UE_API uint16 GetBlendShapeChannelIndexListCount() const override;
	UE_API TArrayView<const uint16> GetBlendShapeChannelIndicesForLOD(uint16 LOD) const override;
	UE_API uint16 GetAnimatedMapCount() const override;
	UE_API FString GetAnimatedMapName(uint16 Index) const override;
	UE_API uint16 GetAnimatedMapIndexListCount() const override;
	UE_API TArrayView<const uint16> GetAnimatedMapIndicesForLOD(uint16 LOD) const override;
	UE_API uint16 GetMeshCount() const override;
	UE_API FString GetMeshName(uint16 Index) const override;
	UE_API uint16 GetMeshIndexListCount() const override;
	UE_API TArrayView<const uint16> GetMeshIndicesForLOD(uint16 LOD) const override;
	UE_API uint16 GetMeshBlendShapeChannelMappingCount() const override;
	UE_API FMeshBlendShapeChannelMapping GetMeshBlendShapeChannelMapping(uint16 Index) const override;
	UE_API TArrayView<const uint16> GetMeshBlendShapeChannelMappingIndicesForLOD(uint16 LOD) const override;
	UE_API FVector GetNeutralJointTranslation(uint16 Index) const override;
	UE_API FVector GetNeutralJointRotation(uint16 Index) const override;
	// Behavior
	UE_API TArrayView<const uint16> GetGUIToRawInputIndices() const override;
	UE_API TArrayView<const uint16> GetGUIToRawOutputIndices() const override;
	UE_API TArrayView<const float> GetGUIToRawFromValues() const override;
	UE_API TArrayView<const float> GetGUIToRawToValues() const override;
	UE_API TArrayView<const float> GetGUIToRawSlopeValues() const override;
	UE_API TArrayView<const float> GetGUIToRawCutValues() const override;
	UE_API uint16 GetPSDCount() const override;
	UE_API TArrayView<const uint16> GetPSDRowIndices() const override;
	UE_API TArrayView<const uint16> GetPSDColumnIndices() const override;
	UE_API TArrayView<const float> GetPSDValues() const override;
	UE_API uint16 GetJointRowCount() const override;
	UE_API uint16 GetJointColumnCount() const override;
	UE_API TArrayView<const uint16> GetJointVariableAttributeIndices(uint16 LOD) const override;
	UE_API uint16 GetJointGroupCount() const override;
	UE_API TArrayView<const uint16> GetJointGroupLODs(uint16 JointGroupIndex) const override;
	UE_API TArrayView<const uint16> GetJointGroupInputIndices(uint16 JointGroupIndex) const override;
	UE_API TArrayView<const uint16> GetJointGroupOutputIndices(uint16 JointGroupIndex) const override;
	UE_API TArrayView<const float> GetJointGroupValues(uint16 JointGroupIndex) const override;
	UE_API TArrayView<const uint16> GetJointGroupJointIndices(uint16 JointGroupIndex) const override;
	UE_API TArrayView<const uint16> GetBlendShapeChannelLODs() const override;
	UE_API TArrayView<const uint16> GetBlendShapeChannelOutputIndices() const override;
	UE_API TArrayView<const uint16> GetBlendShapeChannelInputIndices() const override;
	UE_API TArrayView<const uint16> GetAnimatedMapLODs() const override;
	UE_API TArrayView<const uint16> GetAnimatedMapInputIndices() const override;
	UE_API TArrayView<const uint16> GetAnimatedMapOutputIndices() const override;
	UE_API TArrayView<const float> GetAnimatedMapFromValues() const override;
	UE_API TArrayView<const float> GetAnimatedMapToValues() const override;
	UE_API TArrayView<const float> GetAnimatedMapSlopeValues() const override;
	UE_API TArrayView<const float> GetAnimatedMapCutValues() const override;
	// Geometry
	UE_API uint32 GetVertexPositionCount(uint16 MeshIndex) const override;
	UE_API FVector GetVertexPosition(uint16 MeshIndex, uint32 VertexIndex) const override;
	UE_API TArrayView<const float> GetVertexPositionXs(uint16 MeshIndex) const override;
	UE_API TArrayView<const float> GetVertexPositionYs(uint16 MeshIndex) const override;
	UE_API TArrayView<const float> GetVertexPositionZs(uint16 MeshIndex) const override;
	UE_API uint32 GetVertexTextureCoordinateCount(uint16 MeshIndex) const override;
	UE_API FTextureCoordinate GetVertexTextureCoordinate(uint16 MeshIndex, uint32 TextureCoordinateIndex) const override;
	UE_API uint32 GetVertexNormalCount(uint16 MeshIndex) const override;
	UE_API FVector GetVertexNormal(uint16 MeshIndex, uint32 NormalIndex) const override;
	UE_API uint32 GetFaceCount(uint16 MeshIndex) const override;
	UE_API TArrayView<const uint32> GetFaceVertexLayoutIndices(uint16 MeshIndex, uint32 FaceIndex) const override;
	UE_API uint32 GetVertexLayoutCount(uint16 MeshIndex) const override;
	UE_API FVertexLayout GetVertexLayout(uint16 MeshIndex, uint32 LayoutIndex) const override;
	UE_API uint16 GetMaximumInfluencePerVertex(uint16 MeshIndex) const override;
	UE_API uint32 GetSkinWeightsCount(uint16 MeshIndex) const override;
	UE_API TArrayView<const float> GetSkinWeightsValues(uint16 MeshIndex, uint32 VertexIndex) const override;
	UE_API TArrayView<const uint16> GetSkinWeightsJointIndices(uint16 MeshIndex, uint32 VertexIndex) const override;
	UE_API uint16 GetBlendShapeTargetCount(uint16 MeshIndex) const override;
	UE_API uint16 GetBlendShapeChannelIndex(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	UE_API uint32 GetBlendShapeTargetDeltaCount(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	UE_API FVector GetBlendShapeTargetDelta(uint16 MeshIndex, uint16 BlendShapeTargetIndex, uint32 DeltaIndex) const override;
	UE_API TArrayView<const float> GetBlendShapeTargetDeltaXs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	UE_API TArrayView<const float> GetBlendShapeTargetDeltaYs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	UE_API TArrayView<const float> GetBlendShapeTargetDeltaZs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	UE_API TArrayView<const uint32> GetBlendShapeTargetVertexIndices(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	// Machine Learned Behavior
	UE_API uint16 GetMLControlCount() const override;
	UE_API FString GetMLControlName(uint16 Index) const override;
	UE_API uint16 GetNeuralNetworkCount() const override;
	UE_API uint16 GetNeuralNetworkIndexListCount() const override;
	UE_API TArrayView<const uint16> GetNeuralNetworkIndicesForLOD(uint16 LOD) const override;
	UE_API uint16 GetMeshRegionCount(uint16 MeshIndex) const override;
	UE_API FString GetMeshRegionName(uint16 MeshIndex, uint16 RegionIndex) const override;
	UE_API TArrayView<const uint16> GetNeuralNetworkIndicesForMeshRegion(uint16 MeshIndex, uint16 RegionIndex) const override;
	UE_API TArrayView<const uint16> GetNeuralNetworkInputIndices(uint16 NetIndex) const override;
	UE_API TArrayView<const uint16> GetNeuralNetworkOutputIndices(uint16 NetIndex) const override;
	UE_API uint16 GetNeuralNetworkLayerCount(uint16 NetIndex) const override;
	UE_API EActivationFunction GetNeuralNetworkLayerActivationFunction(uint16 NetIndex, uint16 LayerIndex) const override;
	UE_API TArrayView<const float> GetNeuralNetworkLayerActivationFunctionParameters(uint16 NetIndex, uint16 LayerIndex) const override;
	UE_API TArrayView<const float> GetNeuralNetworkLayerBiases(uint16 NetIndex, uint16 LayerIndex) const override;
	UE_API TArrayView<const float> GetNeuralNetworkLayerWeights(uint16 NetIndex, uint16 LayerIndex) const override;
	// JointBehaviorMetadataReader
	UE_API ETranslationRepresentation GetJointTranslationRepresentation(uint16 JointIndex) const override;
	UE_API ERotationRepresentation GetJointRotationRepresentation(uint16 JointIndex) const override;
	UE_API EScaleRepresentation GetJointScaleRepresentation(uint16 JointIndex) const override;
	// RBFBehavior
	UE_API uint16 GetRBFPoseCount() const override;
	UE_API FString GetRBFPoseName(uint16 PoseIndex) const override;
	UE_API TArrayView<const uint16> GetRBFPoseJointOutputIndices(uint16 PoseIndex) const override;
	UE_API TArrayView<const uint16> GetRBFPoseBlendShapeChannelOutputIndices(uint16 PoseIndex) const override;
	UE_API TArrayView<const uint16> GetRBFPoseAnimatedMapOutputIndices(uint16 PoseIndex) const override;
	UE_API TArrayView<const float> GetRBFPoseJointOutputValues(uint16 PoseIndex) const override;
	UE_API float GetRBFPoseScale(uint16 PoseIndex) const override;
	UE_API uint16 GetRBFPoseControlCount() const override;
	UE_API FString GetRBFPoseControlName(uint16 PoseControlIndex) const override;
	UE_API TArrayView<const uint16> GetRBFPoseInputControlIndices(uint16 PoseIndex) const override;
	UE_API TArrayView<const uint16> GetRBFPoseOutputControlIndices(uint16 PoseIndex) const override;
	UE_API TArrayView<const float> GetRBFPoseOutputControlWeights(uint16 PoseIndex) const override;
	UE_API uint16 GetRBFSolverCount() const override;
	UE_API uint16 GetRBFSolverIndexListCount() const override;
	UE_API TArrayView<const uint16> GetRBFSolverIndicesForLOD(uint16 LOD) const override;
	UE_API FString GetRBFSolverName(uint16 SolverIndex) const override;
	UE_API TArrayView<const uint16> GetRBFSolverRawControlIndices(uint16 SolverIndex) const override;
	UE_API TArrayView<const uint16> GetRBFSolverPoseIndices(uint16 SolverIndex) const override;
	UE_API TArrayView<const float> GetRBFSolverRawControlValues(uint16 SolverIndex) const override;
	UE_API ERBFSolverType GetRBFSolverType(uint16 SolverIndex) const override;
	UE_API float GetRBFSolverRadius(uint16 SolverIndex) const override;
	UE_API EAutomaticRadius GetRBFSolverAutomaticRadius(uint16 SolverIndex) const override;
	UE_API float GetRBFSolverWeightThreshold(uint16 SolverIndex) const override;
	UE_API ERBFDistanceMethod GetRBFSolverDistanceMethod(uint16 SolverIndex) const override;
	UE_API ERBFNormalizeMethod GetRBFSolverNormalizeMethod(uint16 SolverIndex) const override;
	UE_API ERBFFunctionType GetRBFSolverFunctionType(uint16 SolverIndex) const override;
	UE_API ETwistAxis GetRBFSolverTwistAxis(uint16 SolverIndex) const override;
	// TwistSwingBehavior
	UE_API uint16 GetTwistCount() const override;
	UE_API ETwistAxis GetTwistSetupTwistAxis(uint16 TwistIndex) const override;
	UE_API TArrayView<const uint16> GetTwistInputControlIndices(uint16 TwistIndex) const override;
	UE_API TArrayView<const uint16> GetTwistOutputJointIndices(uint16 TwistIndex) const override;
	UE_API TArrayView<const float> GetTwistBlendWeights(uint16 TwistIndex) const override;
	UE_API uint16 GetSwingCount() const override;
	UE_API ETwistAxis GetSwingSetupTwistAxis(uint16 SwingIndex) const override;
	UE_API TArrayView<const uint16> GetSwingInputControlIndices(uint16 SwingIndex) const override;
	UE_API TArrayView<const uint16> GetSwingOutputJointIndices(uint16 SwingIndex) const override;
	UE_API TArrayView<const float> GetSwingBlendWeights(uint16 SwingIndex) const override;
	// MachineLearnedBehaviorExtReader
	UE_API uint16 GetMLTypeCount() const override;
	UE_API uint16 GetMLOperationSetCount(uint16 MLTypeIndex) const override;
	UE_API uint16 GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const override;
	UE_API EMachineLearnedBehaviorOperationType GetMLOperationType(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	UE_API TArrayView<const uint32> GetMLOperationParameters(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	UE_API TArrayView<const uint16> GetMLOperationDependencyOperationSetIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	UE_API TArrayView<const uint16> GetMLOperationDependencyOperationIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	UE_API uint16 GetMLOperationIndexListCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const override;
	UE_API TArrayView<const uint16> GetMLOperationIndicesForLOD(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 LOD) const override;
	UE_API TArrayView<const uint16> GetMLJointsInputIndices() const override;
	UE_API TArrayView<const uint16> GetMLJointsOutputIndices() const override;
	UE_API TArrayView<const uint16> GetMLJointsParameterKeys() const override;
	UE_API TArrayView<const uint16> GetMLJointsParameterValues() const override;
	UE_API void Unload(EDNADataLayer layer) override;

	UE_API const FDNAConfig& GetConfig() const override;

	UE_API dna::Reader* Unwrap() const override;

private:
	TUniquePtr<IDNAReader> ReaderPtr;

};

#undef UE_API
